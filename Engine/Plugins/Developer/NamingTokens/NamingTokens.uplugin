{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "NamingTokens",
	"Description": "Define tokens which can be recognized during string evaluation.",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "NamingTokens",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NamingTokensUI",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NamingTokensUncookedOnly",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		},
		{
			"Name": "NamingTokensEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "EngineAssetDefinitions",
			"Enabled": true
		}
	]
}