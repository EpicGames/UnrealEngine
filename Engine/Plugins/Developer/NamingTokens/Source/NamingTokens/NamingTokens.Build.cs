// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NamingTokens : ModuleRules
{
	public NamingTokens(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine"
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject"
			}
		);

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnrealEd"
				}
			);
		}
	}
}
