// Copyright Epic Games, Inc. All Rights Reserved.

#include "GlobalNamingTokens.h"

#include "Misc/App.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GlobalNamingTokens)

#define LOCTEXT_NAMESPACE "GlobalNamingTokens"

FString UGlobalNamingTokens::GlobalNamespace = TEXT("g");

UGlobalNamingTokens::UGlobalNamingTokens()
{
	Namespace = GlobalNamespace;
}

void UGlobalNamingTokens::OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens)
{
	Tokens.Add({
		TEXT("project"),
		LOCTEXT("ProjectName", "Project Name"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([]() {
			return FText::FromString(FApp::GetProjectName());
		})
	});

	Tokens.Add({
		TEXT("user"),
		LOCTEXT("UserName", "User Name"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([]() {
			return FText::FromString(FApp::GetSessionOwner());
		})
	});

	// Returns empty text intentionally, as this token is used by Asset Tools to substitute in the name of a source object
	Tokens.Add({
		TEXT("source"),
		LOCTEXT("Source", "Source"),
		FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([]() {
			return FText::GetEmpty();
		})
	});

	Tokens.Add({
	    TEXT("yyyy"),
	    LOCTEXT("Year4Digit", "Year (4 digit)"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        return FText::FromString(FString::Printf(TEXT("%04d"), GetCurrentDateTime().GetYear()));
	    })
	});

	Tokens.Add({
	    TEXT("yy"),
	    LOCTEXT("Year2Digit", "Year (2 digit)"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        return FText::FromString(FString::Printf(TEXT("%02d"), GetCurrentDateTime().GetYear() % 100));
	    })
	});

	Tokens.Add({
	    TEXT("Mmm"),
	    LOCTEXT("MonthPascalCase", "3-character Month (Pascal Case)"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        const FString MonthName = GetCurrentDateTime().ToFormattedString(TEXT("%b"));
	        return FText::FromString(MonthName.Left(1).ToUpper() + MonthName.Mid(1).ToLower());
	    })
	});

	Tokens.Add({
	    TEXT("MMM"),
	    LOCTEXT("MonthUpperCase", "3-character Month (UPPERCASE)"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        const FString MonthName = GetCurrentDateTime().ToFormattedString(TEXT("%b"));
	        return FText::FromString(MonthName.ToUpper());
	    })
	});

	Tokens.Add({
	    TEXT("mmm"),
	    LOCTEXT("MonthLowerCase", "3-character Month (lowercase)"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        const FString MonthName = GetCurrentDateTime().ToFormattedString(TEXT("%b"));
	        return FText::FromString(MonthName.ToLower());
	    })
	});

	Tokens.Add({
	    TEXT("mm"),
	    LOCTEXT("Month2Digit", "Month (2 digit)"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        return FText::FromString(FString::Printf(TEXT("%02d"), GetCurrentDateTime().GetMonth()));
	    })
	});

	Tokens.Add({
	    TEXT("Ddd"),
	    LOCTEXT("DayPascalCase", "3-character Day (Pascal Case)"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        const FString DayName = GetCurrentDateTime().ToFormattedString(TEXT("%a"));
	        return FText::FromString(DayName.Left(1).ToUpper() + DayName.Mid(1).ToLower());
	    })
	});

	Tokens.Add({
	    TEXT("DDD"),
	    LOCTEXT("DayUpperCase", "3-character Day (UPPERCASE)"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        const FString DayName = GetCurrentDateTime().ToFormattedString(TEXT("%a"));
	        return FText::FromString(DayName.ToUpper());
	    })
	});

	Tokens.Add({
	    TEXT("ddd"),
	    LOCTEXT("DayLowerCase", "3-character Day (lowercase)"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        const FString DayName = GetCurrentDateTime().ToFormattedString(TEXT("%a"));
	        return FText::FromString(DayName.ToLower());
	    })
	});

	Tokens.Add({
	    TEXT("dd"),
	    LOCTEXT("Day2Digit", "Day (2 digit)"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        return FText::FromString(FString::Printf(TEXT("%02d"), GetCurrentDateTime().GetDay()));
	    })
	});

	Tokens.Add({
	    TEXT("ampm"),
	    LOCTEXT("AMPMLowerCase", "am or pm (lowercase)"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        const FString AMPM = GetCurrentDateTime().ToFormattedString(TEXT("%P"));
	        return FText::FromString(AMPM);
	    })
	});

	Tokens.Add({
	    TEXT("AMPM"),
	    LOCTEXT("AMPMUpperCase", "AM or PM (UPPERCASE)"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        const FString AMPM = GetCurrentDateTime().ToFormattedString(TEXT("%p"));
	        return FText::FromString(AMPM);
	    })
	});

	Tokens.Add({
	    TEXT("12h"),
	    LOCTEXT("Hour12", "Hour (12)"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        return FText::FromString(FString::Printf(TEXT("%02d"), GetCurrentDateTime().GetHour12()));
	    })
	});

	Tokens.Add({
	    TEXT("24h"),
	    LOCTEXT("Hour24", "Hour (24)"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        return FText::FromString(FString::Printf(TEXT("%02d"), GetCurrentDateTime().GetHour()));
	    })
	});

	Tokens.Add({
	    TEXT("min"),
	    LOCTEXT("Minute", "Minute"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        return FText::FromString(FString::Printf(TEXT("%02d"), GetCurrentDateTime().GetMinute()));
	    })
	});

	Tokens.Add({
	    TEXT("sec"),
	    LOCTEXT("Second", "Second"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        return FText::FromString(FString::Printf(TEXT("%02d"), GetCurrentDateTime().GetSecond()));
	    })
	});

	Tokens.Add({
	    TEXT("ms"),
	    LOCTEXT("Millisecond", "Millisecond"),
	    FNamingTokenData::FTokenProcessorDelegateNative::CreateLambda([this]() {
	        return FText::FromString(FString::Printf(TEXT("%02d"), GetCurrentDateTime().GetMillisecond()));
	    })
	});
}

#undef LOCTEXT_NAMESPACE
