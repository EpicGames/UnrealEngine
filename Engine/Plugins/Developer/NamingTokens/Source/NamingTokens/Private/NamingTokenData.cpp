// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokenData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NamingTokenData)

FNamingTokenData::FNamingTokenData(const FString& InTokenKey)
{
	TokenKey = InTokenKey;
	DisplayName = FText::FromString(InTokenKey);
}

FNamingTokenData::FNamingTokenData(const FString& InTokenKey, const FText& InTokenDisplayName, const FTokenProcessorDelegateNative& InTokenProcessor)
{
	DisplayName = InTokenDisplayName;
	TokenKey = InTokenKey;
	TokenProcessorNative = InTokenProcessor;
}

FNamingTokenData::FNamingTokenData(const FString& InTokenKey, const FText& InTokenDisplayName, const FText& InTokenDescription,
	const FTokenProcessorDelegateNative& InTokenProcessor): FNamingTokenData(InTokenKey, InTokenDisplayName, InTokenProcessor)
{
	Description = InTokenDescription;
}

bool FNamingTokenData::Equals(const FNamingTokenData& Other, bool bCaseSensitive) const
{
	return TokenKey.Equals(Other.TokenKey, bCaseSensitive ? ESearchCase::CaseSensitive : ESearchCase::IgnoreCase);
}

bool FNamingTokenData::operator==(const FNamingTokenData& Other) const
{
	return Equals(Other);
}
