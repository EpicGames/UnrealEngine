// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokens.h"

#include "NamingTokensEvaluationData.h"
#include "NamingTokensLog.h"
#include "Utils/NamingTokenUtils.h"
#include "Utils/NamingTokenUtilsPrivate.h"

#include "Engine/Engine.h"
#include "Internationalization/LocKeyFuncs.h"
#include "Internationalization/Regex.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NamingTokens)

#define LOCTEXT_NAMESPACE "NamingTokens"

UNamingTokens::UNamingTokens()
{
}

void UNamingTokens::PostInitProperties()
{
	UObject::PostInitProperties();
	CreateDefaultTokens();
}

UWorld* UNamingTokens::GetWorld() const
{
#if WITH_EDITOR
	if (GEditor)
	{
		if (const FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext())
		{
			return PIEWorldContext->World();
		}

		return GEditor->GetEditorWorldContext(false).World();
	}

	if (IsValid(GWorld))
	{
		return GWorld;
	}
#endif

	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		if (WorldContext.WorldType == EWorldType::Game)
		{
			return WorldContext.World();
		}
	}

	return nullptr;
}

void UNamingTokens::Validate() const
{
	// Namespace validation
	{
		FText ErrorMessage;
		if (!UE::NamingTokens::Utils::ValidateName(Namespace, ErrorMessage))
		{
			UE_LOGF(LogNamingTokens, Error, "NamingTokens Namespace '%ls' has an invalid name. This should be corrected by the owner of this namespace. Error: %ls",
				*Namespace, *ErrorMessage.ToString());
		}
	}

	// Individual token validation
	const TArray<FNamingTokenData> AllTokens = GetAllTokens();
	for (const FNamingTokenData& TokenData : AllTokens)
	{
		FText ErrorMessage;
		if (!UE::NamingTokens::Utils::ValidateName(TokenData.TokenKey, ErrorMessage))
		{
			UE_LOGF(LogNamingTokens, Error, "NamingTokens Token '%ls' under namespace '%ls' has an invalid name. This should be corrected by the owner of this namespace. Error: %ls",
				*TokenData.TokenKey, *Namespace, *ErrorMessage.ToString());
		}
	}
}

void UNamingTokens::CreateDefaultTokens()
{
	DefaultTokens.Empty();
	OnCreateDefaultTokens(DefaultTokens);
}

#if WITH_EDITOR
void UNamingTokens::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UNamingTokens, TestTokenInput))
	{
		EvaluateTestToken();
	}
}

void UNamingTokens::EvaluateTestToken()
{
	const FNamingTokenResultData ResultData = EvaluateTokenText(TestTokenInput);
	TestTokenResult = ResultData.EvaluatedText;
}
#endif

void UNamingTokens::OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens)
{
}

FNamingTokenResultData UNamingTokens::EvaluateTokenText(const FText& InTokenText, const TArray<UObject*>& InContexts)
{
	FNamingTokensEvaluationData EvaluationData;
	EvaluationData.Initialize();
	EvaluationData.Contexts = InContexts;
	return EvaluateTokenText(InTokenText, EvaluationData);
}

FNamingTokenResultData UNamingTokens::EvaluateTokenText(const FText& InTokenText, const FNamingTokensEvaluationData& InEvaluationData)
{
	PreEvaluate(InEvaluationData);
	
	FNamingTokenResultData Result;
	Result.OriginalText = InTokenText;

	const FString NormalizedString = InEvaluationData.bNormalizeInput ? UE::NamingTokens::Utils::Private::NormalizeTokenString(InTokenText.ToString()) : InTokenText.ToString();
	FText FormattedText = InEvaluationData.bNormalizeInput ? FText::FromString(NormalizedString) : InTokenText;

	// Retrieve all unprocessed tokens within this text.
	const TArray<FString> UnprocessedTokens = UE::NamingTokens::Utils::GetTokenKeysFromString(NormalizedString, false);

	TSet<FNamingTokenData> DefaultTokensSet(GetDefaultTokens());
	TSet<FNamingTokenData> CustomTokenSet(CustomTokens);

	TSet<FString, FLocKeySetFuncs> CheckedKeys;
	
	for (const FString& TokenKeyWithNamespace : UnprocessedTokens)
	{
		const FString TokenNamespace = UE::NamingTokens::Utils::GetNamespaceFromTokenKey(TokenKeyWithNamespace);
		if (TokenNamespace.IsEmpty() || TokenNamespace == Namespace)
		{
			// Make sure the token key doesn't include the namespace.
			const FString TokenKey = UE::NamingTokens::Utils::RemoveNamespaceFromTokenKey(TokenKeyWithNamespace);

			if (CheckedKeys.Contains(TokenKey))
			{
				continue;
			}
			CheckedKeys.Add(TokenKey);

			const FNamingTokenData* SelectedToken = nullptr;
			
			FNamingTokenData NamingTokenKeyData(TokenKey);

			// Check case-sensitive match.
			{
				if (FNamingTokenData* DefaultToken = DefaultTokensSet.Find(NamingTokenKeyData))
				{
					SelectedToken = DefaultToken;
				}
				else if (FNamingTokenData* CustomToken = CustomTokenSet.Find(NamingTokenKeyData))
				{
					SelectedToken = CustomToken;
				}
				else
				{
					// Not found in a class definition, check external tokens.
					for (const TTuple<FGuid, TArray<FNamingTokenData>>& ExternalTokenKeyPair : ExternalTokens)
					{
						if (const FNamingTokenData* ExternalToken = ExternalTokenKeyPair.Value.FindByKey(NamingTokenKeyData))
						{
							SelectedToken = ExternalToken;
							break;
						}
					}
				}
			}

			// No exact case-sensitive match found, check case-insensitive.
			if (!SelectedToken && !InEvaluationData.bForceCaseSensitive)
			{
				constexpr bool bCaseSensitive = false;
				if (FNamingTokenData* DefaultToken = DefaultTokens.FindByPredicate([&NamingTokenKeyData](const FNamingTokenData& NamingToken)
				{
					return NamingToken.Equals(NamingTokenKeyData, bCaseSensitive);
				}))
				{
					SelectedToken = DefaultToken;
				}
				else if (FNamingTokenData* CustomToken = CustomTokens.FindByPredicate([&NamingTokenKeyData](const FNamingTokenData& NamingToken)
				{
					return NamingToken.Equals(NamingTokenKeyData, bCaseSensitive);
				}))
				{
					SelectedToken = CustomToken;
				}
				else
				{
					for (const TTuple<FGuid, TArray<FNamingTokenData>>& ExternalTokenKeyPair : ExternalTokens)
					{
						if (const FNamingTokenData* ExternalToken =
							ExternalTokenKeyPair.Value.FindByPredicate([&NamingTokenKeyData](const FNamingTokenData& ExternalTokenKeyPairValue)
						{
							return NamingTokenKeyData.Equals(ExternalTokenKeyPairValue, bCaseSensitive);
						}))
						{
							SelectedToken = ExternalToken;
							break;
						}
					}
				}
			}

			FNamingTokenValueData ValueData;
			
			// If a token match was found, process it, and store its key/value in the output data, along with this NamingTokens object's namespace
			// Otherwise, mark it as un-evaluated, and output no value and the original key and namespace from the input string
			if (SelectedToken)
			{
				// If case-sensitive is set, provide an empty string so ProcessToken will look up the exact namespace for this token. Otherwise,
				// we use what the user provided.
				const FString NamespaceToCheck = InEvaluationData.bForceCaseSensitive ? FString() : TokenNamespace;
				ValueData.TokenValue = ProcessToken(*SelectedToken, NamespaceToCheck, TokenKey, FormattedText);
				ValueData.bWasEvaluated = true;

				ValueData.TokenKey = SelectedToken->TokenKey;
				ValueData.TokenNamespace = Namespace;
			}
			else
			{
				ValueData.TokenValue = FText::GetEmpty();
				ValueData.bWasEvaluated = false;

				ValueData.TokenKey = TokenKey;
				ValueData.TokenNamespace = TokenNamespace;
			}
			
			Result.TokenValues.Add(MoveTemp(ValueData));
		}
	}

	Result.EvaluatedText = FormattedText;

	PostEvaluate(Result);
	
	return Result;
}

FString UNamingTokens::GetFormattedTokensStringForDisplay() const
{
	FString FormattedTokensString;

	for (const FNamingTokenData& Token : GetDefaultTokens())
	{
		FormattedTokensString += FString::Printf(TEXT("%s - %s\n"),
			*UE::NamingTokens::Utils::CreateFormattedToken(Token), *Token.DisplayName.ToString());
	}

	for (const FNamingTokenData& Token : CustomTokens)
	{
		FormattedTokensString += FString::Printf(TEXT("%s - %s\n"),
			*UE::NamingTokens::Utils::CreateFormattedToken(Token), *Token.DisplayName.ToString());
	}
	
	return FormattedTokensString;
}

const TArray<FNamingTokenData>& UNamingTokens::GetDefaultTokens() const
{
	// For now we return the instance version, but we could potentially return the CDO's version.
	return DefaultTokens;
}

const TArray<FNamingTokenData>& UNamingTokens::GetCustomTokens() const
{
	return CustomTokens;
}

TArray<FNamingTokenData>& UNamingTokens::RegisterExternalTokens(FGuid& OutGuid)
{
	OutGuid = FGuid::NewGuid();
	return ExternalTokens.Add(OutGuid);
}

void UNamingTokens::UnregisterExternalTokens(const FGuid& InGuid)
{
	ExternalTokens.Remove(InGuid);
}

bool UNamingTokens::AreExternalTokensRegistered(const FGuid& InGuid) const
{
	return ExternalTokens.Contains(InGuid);
}

TArray<FNamingTokenData>& UNamingTokens::GetExternalTokensChecked(const FGuid& InGuid)
{
	return ExternalTokens.FindChecked(InGuid);
}

TArray<FNamingTokenData> UNamingTokens::GetAllTokens() const
{
	TArray<FNamingTokenData> AllTokens;
	AllTokens.Reserve(DefaultTokens.Num() + CustomTokens.Num());
	AllTokens.Append(DefaultTokens);
	AllTokens.Append(CustomTokens);
	for (const TTuple<FGuid, TArray<FNamingTokenData>>& ExternalTokenKeyPair : ExternalTokens)
	{
		AllTokens.Append(ExternalTokenKeyPair.Value);
	}
	return AllTokens;
}

bool UNamingTokens::IsPrivateNamespace() const
{
	return bIsPrivateNamespace;
}

FDateTime UNamingTokens::GetCurrentDateTime_Implementation() const
{
	return CurrentEvaluationData.CurrentDateTime;
}

void UNamingTokens::PreEvaluate(const FNamingTokensEvaluationData& InEvaluationData)
{
	CurrentEvaluationData = InEvaluationData;
	OnPreEvaluate(InEvaluationData);
	OnPreEvaluateEvent.Broadcast(InEvaluationData);
}

void UNamingTokens::PostEvaluate(const FNamingTokenResultData& InResultData)
{
	OnPostEvaluate();
	OnPostEvaluateEvent.Broadcast(InResultData);
}

void UNamingTokens::OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData)
{
}

void UNamingTokens::OnPostEvaluate_Implementation()
{
}

FText UNamingTokens::ProcessToken(const FNamingTokenData& InToken, const FString& InUserProvidedNamespace, const FString& InUserTokenString, FText& InOutFormattedText)
{
	const FString FormattedToken = *UE::NamingTokens::Utils::CreateFormattedToken(InToken);

	FText ProcessedToken = FText::GetEmpty();

	if (UFunction* BlueprintFunction = FindBlueprintFunctionForToken(InToken))
	{
		struct FFunctionParams
		{
			FText ReturnValue;
		} Params;
		this->ProcessEvent(BlueprintFunction, &Params);
		ProcessedToken = Params.ReturnValue;
	}
	else if (InToken.TokenProcessorNative.IsBound())
	{
		ProcessedToken = InToken.TokenProcessorNative.Execute();
	}
	else
	{
		UE_LOGF(LogNamingTokens, Error, "Could not find function process for token %ls", *FormattedToken);
		return ProcessedToken;
	}

	// Replace both {token} and {namespace:token} in the formatted text.
	FFormatNamedArguments Args;
	Args.Add(InUserTokenString, ProcessedToken);
	Args.Add(UE::NamingTokens::Utils::CombineNamespaceAndTokenKey(InUserProvidedNamespace.IsEmpty()
		? *GetNamespace() : *InUserProvidedNamespace, *InUserTokenString), ProcessedToken);

	InOutFormattedText = FText::Format(InOutFormattedText, Args);
	return ProcessedToken;
}

UFunction* UNamingTokens::FindBlueprintFunctionForToken(const FNamingTokenData& InTokenData) const
{
	const FName FunctionName = InTokenData.FunctionName;
	if (UFunction* Function = GetClass()->FindFunctionByName(FunctionName))
	{
		if (UE::NamingTokens::Utils::ValidateTokenFunction(Function))
		{
			return Function;
		}
		
		UE_LOGF(LogNamingTokens, Warning,
			"Token processor function '%ls' found for token '%ls', but it doesn't have the correct signature",
			*FunctionName.ToString(), *InTokenData.TokenKey);
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
