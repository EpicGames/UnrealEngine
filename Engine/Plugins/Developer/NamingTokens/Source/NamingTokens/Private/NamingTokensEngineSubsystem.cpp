// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokensEngineSubsystem.h"

#include "NamingTokens.h"
#include "NamingTokensEvaluationData.h"
#include "NamingTokensLog.h"
#include "Utils/NamingTokenUtils.h"
#include "Utils/NamingTokenUtilsPrivate.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataTagMap.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Blueprint.h"
#include "Internationalization/Regex.h"
#include "Misc/PackageName.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NamingTokensEngineSubsystem)

TArray<FString> UNamingTokensEngineSubsystem::GetAllNamespaces(bool bIncludePrivateNamespaces) const
{
	TSet<FString> DiscoveredNamespaces;
	
	UE::NamingTokens::Utils::Private::ForEachNamingTokensBlueprint(
		[bIncludePrivateNamespaces, &DiscoveredNamespaces](const FAssetData& AssetData, const FTopLevelAssetPath& ClassObjectPath)
	{
		bool bIsPrivateNamespace = false;
		AssetData.GetTagValue(UNamingTokens::GetIsPrivateNamespacePropertyName(), bIsPrivateNamespace);

		if (!bIsPrivateNamespace || bIncludePrivateNamespaces)
		{
			FString FoundNamespace;
			AssetData.GetTagValue(UNamingTokens::GetNamespacePropertyName(), FoundNamespace);

			DiscoveredNamespaces.Add(FoundNamespace);
		}
	});

	// Find all native classes derived from UNamingTokens, and add their namespaces
	TArray<UClass*> DerivedClasses;
	GetDerivedClasses(UNamingTokens::StaticClass(), DerivedClasses);
	for (const UClass* DerivedClass : DerivedClasses)
	{
		constexpr EClassFlags InvalidClassFlags = CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Abstract;
		if (DerivedClass->IsNative() && !DerivedClass->HasAnyClassFlags(InvalidClassFlags))
		{
			const UNamingTokens* NamingTokens = CastChecked<UNamingTokens>(DerivedClass->GetDefaultObject());
			if (NamingTokens->IsPrivateNamespace() && !bIncludePrivateNamespaces)
			{
				continue;
			}
			
			FString Namespace = NamingTokens->GetNamespace();
			DiscoveredNamespaces.Add(Namespace);
		}
	}

	return DiscoveredNamespaces.Array();
}

UNamingTokensEngineSubsystem::UNamingTokensEngineSubsystem()
	: bIsCacheEnabled(true)
{
}

UNamingTokens* UNamingTokensEngineSubsystem::GetNamingTokens(const FString& InNamespace) const
{
	// Check cache.
	const FString SanitizedNamespace = UE::NamingTokens::Utils::Private::SanitizeTokenOrNamespace(InNamespace);
	UNamingTokens* FoundTokens = GetNamingTokenFromCache(SanitizedNamespace, false);
	
	if (FoundTokens)
	{
		return FoundTokens;
	}

	FoundTokens = GetNamingTokensNative(SanitizedNamespace);

	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	const IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	
	// Check blueprint classes.
	// NOTE: We may want to use OnAssetAdded and cache the asset or load the class then.
	UE::NamingTokens::Utils::Private::ForEachNamingTokensBlueprint(
		[&SanitizedNamespace, &AssetRegistry, &FoundTokens, this](const FAssetData& AssetData, const FTopLevelAssetPath& ClassObjectPath)
	{
		// Now check for the namespace match.
		FString FoundNamespace;
		if (AssetData.GetTagValue(UNamingTokens::GetNamespacePropertyName(), FoundNamespace)
			&& FoundNamespace == SanitizedNamespace)
		{
			// Allow BP children to overwrite native classes.
			auto IsAssetChildOfNativeToken = [&AssetRegistry, &SanitizedNamespace](const FTopLevelAssetPath& InAssetPath, const UNamingTokens* InParentTokens) -> bool
			{
				check(InParentTokens);
				if (InParentTokens->GetClass()->IsNative())
				{
					// We need to make sure this BP is a child of the chosen native class.
					TSet<FTopLevelAssetPath> NativeDerivedClassNames;
					AssetRegistry.GetDerivedClassNames({ InParentTokens->GetClass()->GetClassPathName() },
						{}, NativeDerivedClassNames);

					if (NativeDerivedClassNames.Contains(InAssetPath))
					{
						UE_LOGF(LogNamingTokens, Log, "Using namespace '%ls' of BP child '%ls' instead of native parent '%ls'.",
							*SanitizedNamespace, *InAssetPath.ToString(), *InParentTokens->GetClass()->GetName());
						return true;
					}
				}
				return false;
			};
			
			if (!FoundTokens || IsAssetChildOfNativeToken(ClassObjectPath, FoundTokens))
			{
				const TSoftClassPtr<UNamingTokens> SoftClassPath(ClassObjectPath.ToString());
				FoundTokens = LoadNamingToken(SoftClassPath, SanitizedNamespace);
			}
			else
			{
				UE_LOGF(LogNamingTokens, Warning, "Namespace '%ls' exists more than once in class '%ls' and BP asset '%ls'.",
					*SanitizedNamespace, *FoundTokens->GetClass()->GetName(), *AssetData.PackageName.ToString());
			}
		}
	});

	return FoundTokens;
}

UNamingTokens* UNamingTokensEngineSubsystem::GetNamingTokensNative(const FString& InNamespace) const
{
	const FString SanitizedNamespace = UE::NamingTokens::Utils::Private::SanitizeTokenOrNamespace(InNamespace);
	// Check cache.
	UNamingTokens* FoundTokens = GetNamingTokenFromCache(SanitizedNamespace, true);
	if (FoundTokens)
	{
		return FoundTokens;
	}

	const UClass* TargetClass = UNamingTokens::StaticClass();

	// Check native classes.
	constexpr EClassFlags InvalidClassFlags = CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Abstract;

	TArray<UClass*> DerivedClasses;
	GetDerivedClasses(TargetClass, DerivedClasses);
	for (const UClass* DerivedClass : DerivedClasses)
	{
		if (DerivedClass->IsNative()
			&& DerivedClass->IsChildOf(TargetClass)
			&& !DerivedClass->HasAnyClassFlags(InvalidClassFlags)
			&& !DerivedClass->GetName().StartsWith(TEXT("SKEL_"))
			&& !DerivedClass->GetName().StartsWith(TEXT("REINST_"))
			&& DerivedClass->GetDefaultObject<UNamingTokens>()->GetNamespace() == SanitizedNamespace)
		{
			if (!FoundTokens)
			{
				FoundTokens = LoadNamingToken(DerivedClass, SanitizedNamespace);
			}
			else
			{
				UE_LOGF(LogNamingTokens, Warning, "Namespace '%ls' exists more than once in native class '%ls' and '%ls'.",
						*SanitizedNamespace, *FoundTokens->GetClass()->GetName(), *DerivedClass->GetName());
			}
		}
	}

	return FoundTokens;
}

TArray<UNamingTokens*> UNamingTokensEngineSubsystem::GetMultipleNamingTokens(const TArray<FString>& InNamespaces) const
{
	TArray<UNamingTokens*> Result;
	const TSet<FString> UniqueNamespaces(InNamespaces);
	for (const FString& Namespace : UniqueNamespaces)
	{
		if (UNamingTokens* NamingTokens = GetNamingTokens(Namespace))
		{
			Result.Add(NamingTokens);
		}
	}

	return Result;
}

FNamingTokenResultData UNamingTokensEngineSubsystem::EvaluateTokenText(const FText& InTokenText, const FNamingTokenFilterArgs& InFilter, const TArray<UObject*>& InContexts)
{
	FNamingTokenResultData Result;

	FText ProcessedTokenText = InTokenText;
	Result.EvaluatedText = ProcessedTokenText; // Set so we always have something in case no tokens were evaluated.

	TSet<FString> Namespaces = GetNamingTokenNamespacesFromString(InTokenText.ToString(), InFilter);

	TArray<FNamingTokenValueData> CompletedNamingTokenValueData;
	
	// These external filters are intentionally executed before adding any namespaces from the input FNamingTokenFilterArgs filter. 
	// This allows the caller of this function to guarantee that the namespaces they want included will always be allowed, regardless of other filters that may execute.
	for (const TPair<FName, FFilterNamespace>& DelegatePair : FilterNamespaceDelegates)
	{
		DelegatePair.Value.Execute(Namespaces);
	}

	Namespaces.Append(InFilter.AdditionalNamespacesToInclude); // Tokens could have been written without the namespace but should still be scoped to the filter.

	FNamingTokensEvaluationData EvaluationData;
	EvaluationData.Initialize();
	EvaluationData.Contexts = InContexts;
	EvaluationData.bForceCaseSensitive = InFilter.bForceCaseSensitive;
	EvaluationData.bNormalizeInput = InFilter.bNormalizeInput;
	for (const FString& Namespace : Namespaces)
	{
		DECLARE_DELEGATE_RetVal_OneParam(UNamingTokens*, FGetNamingTokensFunction, const FString&);
		
		FGetNamingTokensFunction GetNamingTokensFunction;
		if (InFilter.bNativeOnly)
		{
			GetNamingTokensFunction.BindUObject(this, &UNamingTokensEngineSubsystem::GetNamingTokensNative);
		}
		else
		{
			GetNamingTokensFunction.BindUObject(this, &UNamingTokensEngineSubsystem::GetNamingTokens);
		}

		if (UNamingTokens* NamingTokens = GetNamingTokensFunction.Execute(Namespace))
		{
			if (NamingTokens->IsPrivateNamespace() && !InFilter.AdditionalNamespacesToInclude.Contains(Namespace))
			{
				continue;
			}
			
			Result = NamingTokens->EvaluateTokenText(ProcessedTokenText, EvaluationData);

			// Properly record individual token evaluation data. The order should be consistent with the order
			// a key was processed in the string. A key may be processed multiple times if it is referenced multiple times, or is not
			// identified in one NamingTokens class but is in another. In the event a future class identifies it, it will update the original
			// key.
			for (const FNamingTokenValueData& NewTokenValue : Result.TokenValues)
			{
				bool bReplacedOldValue = false;
				// Find previously recorded values that were undefined and update them indicating they are no longer undefined.
				for (FNamingTokenValueData& OldTokenValue : CompletedNamingTokenValueData)
				{
					// Token keys can be case sensitive, such as {mmm} and {MMM}, which evaluate differently by design.
					// However, if we are comparing a previous token that was not evaluated with a new token that was,
					// then we can safely ignore case when comparing the token keys, and if they match, then treat the new token as a replacement of the old one.
					const bool bReplacingUnevaluatedToken = !OldTokenValue.bWasEvaluated && NewTokenValue.bWasEvaluated;
					const ESearchCase::Type SearchCase = bReplacingUnevaluatedToken ? ESearchCase::IgnoreCase : ESearchCase::CaseSensitive;

					if (OldTokenValue.TokenKey.Equals(NewTokenValue.TokenKey, SearchCase))
					{
						OldTokenValue = NewTokenValue;
						bReplacedOldValue = true;
						break;
					}
				}
				if (!bReplacedOldValue)
				{
					// Add to the completed naming token value.
					CompletedNamingTokenValueData.Add(NewTokenValue);
				}
			}
			
			ProcessedTokenText = Result.EvaluatedText;
		}
	}

	Result.OriginalText = InTokenText;

	Result.TokenValues = MoveTemp(CompletedNamingTokenValueData);
	
	return Result;
}

FNamingTokenResultData UNamingTokensEngineSubsystem::EvaluateTokenText(const FText& InTokenText, const FNamingTokenFilterArgs& InFilter)
{
	return EvaluateTokenText(InTokenText, InFilter, {});
}

FNamingTokenResultData UNamingTokensEngineSubsystem::EvaluateTokenString(const FString& InTokenString, const FNamingTokenFilterArgs& InFilter, const TArray<UObject*>& InContexts)
{
	return EvaluateTokenText(FText::FromString(InTokenString), InFilter, InContexts);
}

FNamingTokenResultData UNamingTokensEngineSubsystem::EvaluateTokenString(const FString& InTokenString, const FNamingTokenFilterArgs& InFilter)
{
	return EvaluateTokenString(InTokenString, InFilter, {});
}

TArray<FNamingTokenValueData> UNamingTokensEngineSubsystem::EvaluateTokenList(const TArray<FString>& InTokenList, const FNamingTokenFilterArgs& InFilter,
                                                                           const TArray<UObject*>& InContexts)
{
	const FString TokenStringCombined = FString::JoinBy(InTokenList, TEXT(", "),
		[](const FString& InToken)
		{
			return UE::NamingTokens::Utils::CreateFormattedToken(FNamingTokenData(InToken));
		});
	FNamingTokenResultData Result = EvaluateTokenString(TokenStringCombined, InFilter, InContexts);
	return Result.TokenValues;
}

TArray<FNamingTokenValueData> UNamingTokensEngineSubsystem::EvaluateTokenList(const TArray<FString>& InTokenList, const FNamingTokenFilterArgs& InFilter)
{
	return EvaluateTokenList(InTokenList, InFilter, {});
}

void UNamingTokensEngineSubsystem::RegisterGlobalNamespace(const FString& InNamespace)
{
	if (IsGlobalNamespaceRegistered(InNamespace))
	{
		const FString UnregisterFunctionName = GET_FUNCTION_NAME_CHECKED(UNamingTokensEngineSubsystem, UnregisterGlobalNamespace).ToString();
		UE_LOGF(LogNamingTokens, Error, "NamingTokens Namespace '%ls' is already registered as a global namespace. Call '%ls' first if you wish to overwrite the namespace.",
			 *InNamespace, *UnregisterFunctionName);
		return;
	}
	FText ErrorMessage;
	if (UE::NamingTokens::Utils::ValidateName(InNamespace, ErrorMessage))
	{
		GlobalNamespaces.Add(InNamespace);
	}
	else
	{
		UE_LOGF(LogNamingTokens, Error, "NamingTokens Namespace '%ls' cannot be registered. Error: %ls",
			*InNamespace, *ErrorMessage.ToString());
	}
}

void UNamingTokensEngineSubsystem::UnregisterGlobalNamespace(const FString& InNamespace)
{
	GlobalNamespaces.Remove(InNamespace);
}

bool UNamingTokensEngineSubsystem::IsGlobalNamespaceRegistered(const FString& InNamespace) const
{
	return GlobalNamespaces.Contains(InNamespace);
}

TArray<FString> UNamingTokensEngineSubsystem::GetGlobalNamespaces() const
{
	return GlobalNamespaces.Array();
}

void UNamingTokensEngineSubsystem::RegisterNamespaceFilter(const FName OwnerName, FFilterNamespace Delegate)
{
	FilterNamespaceDelegates.Add(OwnerName, MoveTemp(Delegate));
}

void UNamingTokensEngineSubsystem::UnregisterNamespaceFilter(const FName OwnerName)
{
	FilterNamespaceDelegates.Remove(OwnerName);
}

TSet<FString> UNamingTokensEngineSubsystem::GetNamingTokenNamespacesFromString(const FString& InTokenString, const FNamingTokenFilterArgs& InFilter) const
{
	TSet<FString> Namespaces;

	const FRegexPattern Pattern(FString::Printf(TEXT(R"(\{\s*([a-zA-Z0-9_ ]+)\s*%s\s*[a-zA-Z0-9_ ]+\s*\})"), *UE::NamingTokens::Utils::GetNamespaceDelimiter()));
	FRegexMatcher Matcher(Pattern, InTokenString);
	
	while (Matcher.FindNext())
	{
		const FString Namespace = Matcher.GetCaptureGroup(1);
		Namespaces.Add(Namespace);
	}

	// Add any global namespaces.
	if (InFilter.bIncludeGlobal)
	{
		Namespaces.Append(GlobalNamespaces);
	}
	
	return Namespaces;	
}

FString UNamingTokensEngineSubsystem::GetFormattedTokensStringForDisplay(const FNamingTokenFilterArgs& InFilter) const
{
	TSet<FString> Namespaces = TSet<FString>(InFilter.AdditionalNamespacesToInclude);

	if (InFilter.bIncludeGlobal)
	{
		Namespaces.Append(GlobalNamespaces.Array());
	}

	FString Result;

	for (const FString& Namespace : Namespaces)
	{
		if (const UNamingTokens* NamingTokens = GetNamingTokens(Namespace))
		{
			Result += NamingTokens->GetFormattedTokensStringForDisplay();
		}
	}

	return Result;
}

void UNamingTokensEngineSubsystem::ClearCachedNamingTokens()
{
	FScopeLock Lock(&CachedNamingTokensMutex);
	CachedNamingTokens.Empty();
}

bool UNamingTokensEngineSubsystem::IsCacheEnabled() const
{
	return bIsCacheEnabled;
}

void UNamingTokensEngineSubsystem::SetCacheEnabled(bool bEnabled)
{
	bIsCacheEnabled = bEnabled;
	if (!bIsCacheEnabled)
	{
		ClearCachedNamingTokens();
	}
}

UNamingTokens* UNamingTokensEngineSubsystem::LoadNamingToken(const TSoftClassPtr<UNamingTokens>& InTokensClass, const FString& InNamespace) const
{
	if (const UClass* Class = InTokensClass.LoadSynchronous())
	{
		UNamingTokens* NamingTokens = NewObject<UNamingTokens>(GetTransientPackage(), Class);
		// Initial validation on load. This will log any errors.
		NamingTokens->Validate();
		if (ensure(NamingTokens->GetNamespace() == InNamespace))
		{
			FScopeLock Lock(&CachedNamingTokensMutex);
			if (IsCacheEnabled())
			{
				CachedNamingTokens.Add(InNamespace, NamingTokens);
			}
			return NamingTokens;
		}
	}

	return nullptr;
}

UNamingTokens* UNamingTokensEngineSubsystem::GetNamingTokenFromCache(const FString& InNamespace, bool bNativeOnly) const
{
	FScopeLock Lock(&CachedNamingTokensMutex);
	const FString SanitizedNamespace = UE::NamingTokens::Utils::Private::SanitizeTokenOrNamespace(InNamespace);
	if (IsCacheEnabled())
	{
		if (const TObjectPtr<UNamingTokens>* Tokens = CachedNamingTokens.Find(SanitizedNamespace))
		{
			if ((*Tokens) == nullptr)
			{
				// NamingTokens may have been deleted, with a new asset using the same namespace then added afterward.
				CachedNamingTokens.Remove(SanitizedNamespace);
				return nullptr;
			}
			
			if (!bNativeOnly)
			{
				return *Tokens;
			}

			if (bNativeOnly && (*Tokens)->GetClass()->IsNative())
			{
				return *Tokens;
			}
		}
	}

	return nullptr;
}
