// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokensEvaluationData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NamingTokensEvaluationData)

FNamingTokensEvaluationData::FNamingTokensEvaluationData()
{
}

void FNamingTokensEvaluationData::Initialize()
{
	CurrentDateTime = FDateTime::Now();
}
