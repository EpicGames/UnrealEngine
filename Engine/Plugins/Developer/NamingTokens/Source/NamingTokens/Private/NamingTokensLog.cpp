// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokensLog.h"

DEFINE_LOG_CATEGORY(LogNamingTokens);
