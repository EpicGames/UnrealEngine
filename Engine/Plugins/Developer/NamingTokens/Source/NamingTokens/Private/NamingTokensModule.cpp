// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokensModule.h"

#include "GlobalNamingTokens.h"
#include "NamingTokensEngineSubsystem.h"

#include "Engine/Engine.h"
#include "Misc/CoreDelegates.h"

#define LOCTEXT_NAMESPACE "FNamingTokensModule"

void FNamingTokensModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FNamingTokensModule::OnPostEngineInit);
}

void FNamingTokensModule::ShutdownModule()
{
	if (GEngine)
	{
		if (UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>())
		{
			NamingTokensSubsystem->UnregisterGlobalNamespace(UGlobalNamingTokens::GetGlobalNamespace());
		}
	}
	
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
}

void FNamingTokensModule::OnPostEngineInit()
{
	if (GEngine)
	{
		if (UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>())
		{
			if (!NamingTokensSubsystem->IsGlobalNamespaceRegistered(UGlobalNamingTokens::GetGlobalNamespace()))
			{
				NamingTokensSubsystem->RegisterGlobalNamespace(UGlobalNamingTokens::GetGlobalNamespace());
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FNamingTokensModule, NamingTokens)