// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "INamingTokensModule.h"

class FNamingTokensModule : public INamingTokensModule
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void OnPostEngineInit();
};
