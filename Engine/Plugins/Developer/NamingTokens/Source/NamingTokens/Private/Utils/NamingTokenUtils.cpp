// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/NamingTokenUtils.h"

#include "Utils/NamingTokenUtilsPrivate.h"
#include "NamingTokens.h"

#include "Internationalization/LocKeyFuncs.h"
#include "Internationalization/Regex.h"

UFunction* UE::NamingTokens::Utils::GetProcessTokenFunctionSignature()
{
	UFunction* FunctionSignature = UNamingTokens::StaticClass()->FindFunctionByName(UNamingTokens::GetProcessTokenTemplateFunctionName());
	check(FunctionSignature);
	return FunctionSignature;
}

bool UE::NamingTokens::Utils::ValidateTokenFunction(const UFunction* InFunction)
{
	check(InFunction);
	const UFunction* FunctionSignature = GetProcessTokenFunctionSignature();
	return InFunction->IsSignatureCompatibleWith(FunctionSignature,
				/* Ignore CPF_ReturnParm for inconsistency between native and BP return params. */
				UFunction::GetDefaultIgnoredSignatureCompatibilityFlags() | CPF_ReturnParm)
				&& InFunction->HasAnyFunctionFlags(FUNC_BlueprintCallable)
				&& !InFunction->HasAnyFunctionFlags(FUNC_Private);
}

FString UE::NamingTokens::Utils::CreateFormattedToken(const FNamingTokenData& InToken)
{
	return FString::Printf(TEXT("{%s}"), *InToken.TokenKey);
}

FString UE::NamingTokens::Utils::GetNamespaceDelimiter()
{
	static const FString Delimiter(TEXT(":"));

	return Delimiter;
}

TArray<FString> UE::NamingTokens::Utils::GetTokenKeysFromString(const FString& InTokenString, bool bNormalizeString)
{
	const FString NormalizedString = bNormalizeString ? Private::NormalizeTokenString(InTokenString) : InTokenString;
	static const FString PatternString = Private::GetTokenPatternString();
	static const FRegexPattern Pattern(PatternString);
	
	TSet<FString, FLocKeySetFuncs> Tokens;
	FRegexMatcher Matcher(Pattern, NormalizedString);

	while (Matcher.FindNext())
	{
		FString Token = Matcher.GetCaptureGroup(1);
		Token.RemoveSpacesInline();
		Tokens.Add(Token);
	}

	return Tokens.Array();
}

bool UE::NamingTokens::Utils::IsTokenInString(const FString& InTokenKey, const FString& InTokenString)
{
	static const FString PatternString = Private::GetTokenPatternString();
	static const FRegexPattern Pattern(PatternString);

	FRegexMatcher Matcher(Pattern, InTokenString);

	while (Matcher.FindNext())
	{
		const FString Token = Matcher.GetCaptureGroup(1);
		const FString TokenNoNamespace = RemoveNamespaceFromTokenKey(Token);
		if (TokenNoNamespace.Equals(InTokenKey))
		{
			return true;
		}
	}

	return false;
}

FString UE::NamingTokens::Utils::GetNamespaceFromTokenKey(const FString& InTokenKey)
{
	FString LeftPart, RightPart;
	InTokenKey.Split(GetNamespaceDelimiter(), &LeftPart, &RightPart);

	return LeftPart;
}

FString UE::NamingTokens::Utils::RemoveNamespaceFromTokenKey(const FString& InTokenKey)
{
	FString Result = InTokenKey;
	FString LeftPart, RightPart;
	if (InTokenKey.Split(GetNamespaceDelimiter(), &LeftPart, &RightPart))
	{
		Result = RightPart;
	}
	return Result;
}

FString UE::NamingTokens::Utils::CombineNamespaceAndTokenKey(const FString& InNamespace, const FString& InTokenKey)
{
	if (InNamespace.IsEmpty())
	{
		return InTokenKey;
	}
	
	return FString::Printf(TEXT("%s%s%s"), *InNamespace, *GetNamespaceDelimiter(), *InTokenKey);
}

bool UE::NamingTokens::Utils::ValidateName(const FString& InName, FText& OutErrorMessage)
{
	OutErrorMessage = FText::GetEmpty();

	if (InName.IsEmpty())
	{
		OutErrorMessage = NSLOCTEXT("NamingTokenUtils", "StringEmpty", "String is empty");
	}
	else
	{
		TSet<TCHAR> UniqueInvalidChars;
		for (const TCHAR& Char : InName)
		{
			// We may want to eventually use FName::IsValidXName with INVALID_OBJECTNAME_CHARACTERS or variant.
			// For right now, this explicitly allows the 3 types we want and reports invalid characters.
			if (!FChar::IsAlpha(Char) && !FChar::IsDigit(Char) && Char != TEXT('_'))
			{
				UniqueInvalidChars.Add(Char);
			}
		}

		if (UniqueInvalidChars.Num() > 0)
		{
			TArray<FString> InvalidStrings;
			for (const TCHAR& Char : UniqueInvalidChars)
			{
				InvalidStrings.Add(FString::Printf(TEXT("'%c'"), Char));
			}
			OutErrorMessage = FText::FromString(TEXT("Invalid characters: ") + FString::Join(InvalidStrings, TEXT(", ")));
		}
	}
	
	return OutErrorMessage.IsEmpty();
}
