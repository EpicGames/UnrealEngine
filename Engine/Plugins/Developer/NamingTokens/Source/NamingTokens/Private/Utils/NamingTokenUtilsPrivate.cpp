// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokenUtilsPrivate.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataTagMap.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/Blueprint.h"
#include "Internationalization/Regex.h"
#include "Misc/PackageName.h"
#include "NamingTokens.h"
#include "Utils/NamingTokenUtils.h"

void UE::NamingTokens::Utils::Private::ForEachNamingTokensBlueprint(
	const TFunction<void(const FAssetData& AssetData, const FTopLevelAssetPath& ClassObjectPath)>& Function)
{
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	const IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();
	
	FARFilter Filter;
	Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;
	Filter.bRecursivePaths = true;

	TArray<FAssetData> BlueprintAssets;
	AssetRegistry.GetAssets(Filter, BlueprintAssets);

	TSet<FTopLevelAssetPath> DerivedClassNames;
	AssetRegistry.GetDerivedClassNames({ UNamingTokens::StaticClass()->GetClassPathName() }, {}, DerivedClassNames);
	
	for (const FAssetData& NamingTokenAssetData : BlueprintAssets)
	{
		// Narrow down to only our assets.
		const FAssetDataTagMapSharedView::FFindTagResult Result = NamingTokenAssetData.TagsAndValues.FindTag(TEXT("GeneratedClass"));
		if (Result.IsSet())
		{
			const FString& GeneratedClassPathPtr = Result.GetValue();
			const FTopLevelAssetPath ClassObjectPath(FPackageName::ExportTextPathToObjectPath(*GeneratedClassPathPtr));

			if (DerivedClassNames.Contains(ClassObjectPath))
			{
				Function(NamingTokenAssetData, ClassObjectPath);
			}
		}
	}
}

FString UE::NamingTokens::Utils::Private::NormalizeTokenString(const FString& InTokenString)
{
	const FRegexPattern Pattern(TEXT(R"(\{([^}]*)\})"));
	FRegexMatcher Matcher(Pattern, InTokenString);

	FString Output;
	int32 LastIndex = 0;

	while (Matcher.FindNext())
	{
		const int32 Start = Matcher.GetMatchBeginning();
		const int32 End = Matcher.GetMatchEnding();

		Output += InTokenString.Mid(LastIndex, Start - LastIndex);

		FString Inner = Matcher.GetCaptureGroup(1);
		Inner.ReplaceInline(TEXT(" "), TEXT(""));
		Inner.ReplaceInline(TEXT("\t"), TEXT(""));
		Inner.ReplaceInline(TEXT("\n"), TEXT(""));
		Inner.ReplaceInline(TEXT("\r"), TEXT(""));
		
		Output += FString::Printf(TEXT("{%s}"), *Inner);

		LastIndex = End;
	}

	Output += InTokenString.Mid(LastIndex);
	return Output;
}

FString UE::NamingTokens::Utils::Private::SanitizeTokenOrNamespace(const FString& InString)
{
	FString SanitizedString = InString;
	SanitizedString.RemoveSpacesInline();
	
	return SanitizedString;
}

FString UE::NamingTokens::Utils::Private::GetTokenPatternString()
{
	return FString::Printf(TEXT(R"(\{\s*((?:[a-zA-Z0-9_]+%s)*[a-zA-Z0-9_]+)\s*\})"), *GetNamespaceDelimiter());
}
