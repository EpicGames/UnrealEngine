// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"

struct FAssetData;
struct FTopLevelAssetPath;

namespace UE::NamingTokens::Utils::Private
{
	/** Iterate each Naming Tokens blueprint asset, executing a function for each qualifying asset. */
	void ForEachNamingTokensBlueprint(const TFunction<void(const FAssetData& AssetData, const FTopLevelAssetPath& ClassObjectPath)>& Function);
	
	/** Preprocess a string removing any spaces within braces. */
	FString NormalizeTokenString(const FString& InTokenString);
	
	/** Remove spaces from within tokens or namespaces. */
	FString SanitizeTokenOrNamespace(const FString& InString);
	
	/** Retrieve the token regex. */
	FString GetTokenPatternString();
}
