// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NamingTokens.h"

#include "GlobalNamingTokens.generated.h"

#define UE_API NAMINGTOKENS_API

/**
 * Default global tokens accessed project wide.
 */
UCLASS(MinimalAPI)
class UGlobalNamingTokens : public UNamingTokens
{
	GENERATED_BODY()

public:
	UE_API UGlobalNamingTokens();

	static const FString& GetGlobalNamespace() { return GlobalNamespace; }
	
protected:
	/** Define any default tokens. */
	UE_API virtual void OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens) override;

private:
	/** The global namespace to register this naming token as. */
	static UE_API FString GlobalNamespace;
};

#undef UE_API
