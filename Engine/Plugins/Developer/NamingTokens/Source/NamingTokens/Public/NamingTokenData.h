// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "NamingTokenData.generated.h"

#define UE_API NAMINGTOKENS_API

class UNamingTokens;

USTRUCT(BlueprintType)
struct FNamingTokenValueData
{
	GENERATED_BODY()
	
	/** The token key. */
	UPROPERTY(BlueprintReadOnly, Category = "NamingTokens")
	FString TokenKey;
	/** The namespace of the token key. */
	UPROPERTY(BlueprintReadOnly, Category = "NamingTokens")
	FString TokenNamespace;
	/** The evaluated token text. */
	UPROPERTY(BlueprintReadOnly, Category = "NamingTokens")
	FText TokenValue;
	/** If the token was able to be evaluated. */
	UPROPERTY(BlueprintReadOnly, Category = "NamingTokens")
	bool bWasEvaluated = false;
};

/** Evaluated results from a template file string. */
USTRUCT(BlueprintType)
struct FNamingTokenResultData
{
	GENERATED_BODY()
	
	/** Original text without any modifications. */
	UPROPERTY(BlueprintReadOnly, Category = "NamingTokens")
	FText OriginalText;
	/** The full text with evaluated tokens. */
	UPROPERTY(BlueprintReadOnly, Category = "NamingTokens")
	FText EvaluatedText;
	/** The result of individual tokens, in the order they appear in OriginalText. */
	UPROPERTY(BlueprintReadOnly, Category = "NamingTokens")
	TArray<FNamingTokenValueData> TokenValues;
};

USTRUCT(BlueprintType)
struct FNamingTokenData
{
	GENERATED_BODY()

public:
	FNamingTokenData() = default;

	DECLARE_DELEGATE_RetVal(FText, FTokenProcessorDelegateNative);

	UE_API explicit FNamingTokenData(const FString& InTokenKey);
	UE_API FNamingTokenData(const FString& InTokenKey, const FText& InTokenDisplayName, const FTokenProcessorDelegateNative& InTokenProcessor);
	UE_API FNamingTokenData(const FString& InTokenKey, const FText& InTokenDisplayName,
	                 const FText& InTokenDescription, const FTokenProcessorDelegateNative& InTokenProcessor);

	/**
	 * The key of the token to use.
	 * This is what the text must match in order to be evaluated. Brackets are automatically added and do not need to be included.
	 *
	 * @note Must contain alphanumeric and '_' characters only and cannot be empty.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "NamingTokens")
	FString TokenKey;

	/** The friendly display name of the token. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "NamingTokens")
	FText DisplayName;

	/** A description of the token. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "NamingTokens")
	FText Description;

	/** The function to use to evaluate the token. Only needed for blueprint implementations. */
	UPROPERTY(EditAnywhere, Category = "NamingTokens", meta = (NoResetToDefault))
	FName FunctionName;

	/** The native delegate to execute to evaluate the function. If FunctionName is set then this is not used. */
	FTokenProcessorDelegateNative TokenProcessorNative;

	UE_API bool operator==(const FNamingTokenData& Other) const;

	/** Checks for equality dependent on case sensitivity. */
	UE_API bool Equals(const FNamingTokenData& Other, bool bCaseSensitive = true) const;
	
	friend FORCEINLINE uint32 GetTypeHash(const FNamingTokenData& Item)
	{
		const uint32 Hash = FCrc::StrCrc32(*Item.TokenKey);
		return Hash;
	}
};

#undef UE_API
