// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NamingTokenData.h"
#include "NamingTokensEvaluationData.h"

#include "NamingTokens.generated.h"

#define UE_API NAMINGTOKENS_API

/**
 * Subclass to define naming tokens to use for a specific tool or project.
 */
UCLASS(MinimalAPI, Blueprintable, Abstract)
class UNamingTokens : public UObject
{
	GENERATED_BODY()

public:
	UE_API UNamingTokens();

	UE_API virtual void PostInitProperties() override;
	UE_API virtual UWorld* GetWorld() const override;

	/** Validate internal values including namespace and token keys. */
	UE_API void Validate() const;
	
	/** Create any default tokens. */
	UE_API void CreateDefaultTokens();

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	/** Evaluate the test token input. */
	UE_API void EvaluateTestToken();
#endif

protected:
	/** Define any default tokens. */
	UE_API virtual void OnCreateDefaultTokens(TArray<FNamingTokenData>& Tokens);

public:
	/** Evaluate token text. Creates EvaluationData. */
	UE_API FNamingTokenResultData EvaluateTokenText(const FText& InTokenText, const TArray<UObject*>& InContexts = {});

	/** Evaluate token text for a given EvaluationData. */
	UE_API FNamingTokenResultData EvaluateTokenText(const FText& InTokenText, const FNamingTokensEvaluationData& InEvaluationData);
	
	/** Creates a friendly display string of all tokens. */
	UE_API FString GetFormattedTokensStringForDisplay() const;

	/** Retrieve the default tokens. */
	UE_API const TArray<FNamingTokenData>& GetDefaultTokens() const;
	
	/** Retrieve the custom tokens. */
	UE_API const TArray<FNamingTokenData>& GetCustomTokens() const;

	/**
	 * Register an external token array which is managed from a caller. This is transient data.
	 * To retrieve the array, call GetExternalTokensChecked and pass in the Guid.
	 *
	 * @param OutGuid The new guid these tokens are registered under.
	 *
	 * @return A reference to the token array. This reference may be invalidated if another array is registered.
	 */
	UE_API TArray<FNamingTokenData>& RegisterExternalTokens(FGuid& OutGuid);

	/**
	 * Unregister and clear out external tokens.
	 * 
	 * @param InGuid The guid to unregister.
	 */
	UE_API void UnregisterExternalTokens(const FGuid& InGuid);
	
	/**
	 * Check if external tokens are registered for a guid.
	 *
	 * @param InGuid The guid which was registered for these external tokens.
	 * @return True if the tokens are registered, false if they are not registered.
	 */
	UE_API bool AreExternalTokensRegistered(const FGuid& InGuid) const;

	/**
	 * Retrieve the external tokens. Tokens must be registered and exist.
	 *
	 * @param InGuid The guid which was registered for these external tokens.
	 * @return A reference to the token array. This reference may be invalidated if another array is registered.
	 */
	UE_API TArray<FNamingTokenData>& GetExternalTokensChecked(const FGuid& InGuid);
	
	/** Retrieve all tokens. */
	UE_API TArray<FNamingTokenData> GetAllTokens() const;

	/** Retrieve the naming token's namespace. */
	const FString& GetNamespace() const { return Namespace; }
	/** The name of the namespace property. */
	static FName GetNamespacePropertyName() { return GET_MEMBER_NAME_CHECKED(UNamingTokens, Namespace); }
	/** The name of IsPrivateNamespace property. */
	static FName GetIsPrivateNamespacePropertyName() { return GET_MEMBER_NAME_CHECKED(UNamingTokens, bIsPrivateNamespace); }

	/** Retrieve the namespace's friendly display name. */
	const FText& GetNamespaceDisplayName() const { return NamespaceDisplayName; }
	
	/** Delegate when pre evaluate is called. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPreEvaluate, const FNamingTokensEvaluationData& InEvaluationData);
	FOnPreEvaluate& GetOnPreEvaluateEvent() { return OnPreEvaluateEvent; }
	/** Delegate when post evaluate is called. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostEvaluate, const FNamingTokenResultData& InResultData);
	FOnPostEvaluate& GetOnPostEvaluateEvent() { return OnPostEvaluateEvent; }

	/** Retrieve the current datetime. By default, this uses shared data so results are consistent across runs. */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "NamingTokens")
	UE_API FDateTime GetCurrentDateTime() const;

	/** If this namespace is private. */
	UFUNCTION(BlueprintCallable, Category = "NamingTokens")
	UE_API bool IsPrivateNamespace() const;
	
protected:
	UE_API virtual FDateTime GetCurrentDateTime_Implementation() const;

public:
	static FName GetProcessTokenTemplateFunctionName() { return GET_FUNCTION_NAME_CHECKED(UNamingTokens, ProcessTokenTemplateFunction); } 
	
	static FName GetOnPreEvaluateFunctionName() { return GET_FUNCTION_NAME_CHECKED(UNamingTokens, OnPreEvaluate); }
	static FName GetOnPostEvaluateFunctionName() { return GET_FUNCTION_NAME_CHECKED(UNamingTokens, OnPostEvaluate); }
	
private:
	/** Template function for us to dynamically create subclass graphs from matching this signature. */
	UFUNCTION()
	FText ProcessTokenTemplateFunction() { return FText::GetEmpty(); }

protected:
	/**
	 * Called prior to token evaluation.
	 * 
	 * @param InEvaluationData Shared information across namespaces.
	 */
	UE_API void PreEvaluate(const FNamingTokensEvaluationData& InEvaluationData);
	
	/**
	 * Called after all tokens have evaluated.
	 */
	UE_API void PostEvaluate(const FNamingTokenResultData& InResultData);
	
	/**
	 * Called prior to evaluation. Allows consistent data to be set up for each token evaluation.
	 * 
	 * This is important if the data is temporally sensitive and could change between evaluating
	 * multiple tokens in a string, such as a high resolution timer.
	 * 
	 * @param InEvaluationData Contains generic shared data for this evaluation.
	 */
	UFUNCTION(BlueprintNativeEvent, Category = "NamingTokens")
	UE_API void OnPreEvaluate(const FNamingTokensEvaluationData& InEvaluationData);
	UE_API virtual void OnPreEvaluate_Implementation(const FNamingTokensEvaluationData& InEvaluationData);
	
	/** Called after evaluation. Used so any cleanup can occur. */
	UFUNCTION(BlueprintNativeEvent, Category = "NamingTokens")
	UE_API void OnPostEvaluate();
	UE_API virtual void OnPostEvaluate_Implementation();

private:
	/** Delegate when pre evaluate is called. */
	FOnPreEvaluate OnPreEvaluateEvent;
	/** Delegate when post evaluate is called. */
	FOnPostEvaluate OnPostEvaluateEvent;
	
	/** Process a token if is defined and return evaluated text. */
	UE_API FText ProcessToken(const FNamingTokenData& InToken, const FString& InUserProvidedNamespace, const FString& InUserTokenString, FText& InOutFormattedText);
	
	/** Return a blueprint function for a token processor if it exists. */
	UE_API UFunction* FindBlueprintFunctionForToken(const FNamingTokenData& InTokenData) const;

private:
	/** The default tokens defined by this class. */
	TArray<FNamingTokenData> DefaultTokens;

	/** External and temporary instance tokens which can be filled in by tools supporting unrecognized tokens. */
	TMap<FGuid, TArray<FNamingTokenData>> ExternalTokens;
	
protected:
	/** User defined tokens. */
	UPROPERTY(EditDefaultsOnly, Category = "NamingTokens", meta = (TitleProperty="TokenKey"))
	TArray<FNamingTokenData> CustomTokens;

	/** Cached shared data for this evaluation. */
	UPROPERTY(BlueprintReadOnly, Category = "NamingTokens")
	FNamingTokensEvaluationData CurrentEvaluationData;
	
	/**
	 * The namespace to identify this token.
	 * 
	 * @note Must contain alphanumeric and '_' characters only and cannot be empty.
	 */
	UPROPERTY(EditDefaultsOnly, AssetRegistrySearchable, Category = "NamingTokens")
	FString Namespace;

	/**
	 * The full display name of the namespace to use in UI and filtering.
	 */
	UPROPERTY(EditDefaultsOnly, AssetRegistrySearchable, Category = "NamingTokens")
	FText NamespaceDisplayName;

	/**
	 * If the namespace is considered private. This prevents this namespace from being gathered for display
	 * or processing unless explicitly specified.
	 */
	UPROPERTY(AdvancedDisplay, EditDefaultsOnly, AssetRegistrySearchable, Category = "NamingTokens")
	bool bIsPrivateNamespace = false;
	
#if WITH_EDITORONLY_DATA
private:
	/** Enter a sample string using your tokens to output an evaluated result to TestTokenResult. */
	UPROPERTY(EditDefaultsOnly, Transient, Category = "NamingTokens")
	FText TestTokenInput;

	/** An evaluated text result of your token data. */
	UPROPERTY(VisibleDefaultsOnly, Transient, Category = "NamingTokens")
	FText TestTokenResult;
#endif
};

#undef UE_API
