// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"

#include "NamingTokenData.h"

#include "NamingTokensEngineSubsystem.generated.h"

#define UE_API NAMINGTOKENS_API

class UNamingTokens;

DECLARE_DELEGATE_OneParam(FFilterNamespace, TSet<FString>& /*Namespaces*/);

USTRUCT(BlueprintType)
struct FNamingTokenFilterArgs
{
	GENERATED_BODY()
	
	/**
	 * Namespaces to always be included during evaluation. Namespaces added here won't require the 'namespace' string prefixed to tokens.
	 * This does not filter out any namespaces.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NamingTokens)
	TArray<FString> AdditionalNamespacesToInclude;
	
	/** Include global namespaces. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NamingTokens)
	bool bIncludeGlobal = true;

	/** When false, we fall back to case-insensitive if an exact match isn't found. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NamingTokens)
	bool bForceCaseSensitive = false;

	/** When false, we additionally look for blueprint naming tokens. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NamingTokens)
	bool bNativeOnly = false;
	
	/**
	 * If input data should be normalized first, such as removing white spaces in tokens.
	 * A better user experience typing in tokens, but may impact localization data.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NamingTokens)
	bool bNormalizeInput = true;
};

/**
 * An editor subsystem for registering global tokens and evaluating strings across the entire project.
 */
UCLASS(MinimalAPI)
class UNamingTokensEngineSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	UE_API UNamingTokensEngineSubsystem();
	
	/**
	 * Lookup naming tokens given a namespace. This will look first in cached naming tokens,
	 * then native classes, then blueprint classes. Call ClearCachedNamingTokens to reset the cache
	 * and force a full lookup.
	 *
	 * @param InNamespace The namespace of the tokens.
	 * @return The found Naming Tokens object, or nullptr.
	 */
	UFUNCTION(BlueprintCallable, Category = "NamingTokens")
	UE_API UNamingTokens* GetNamingTokens(const FString& InNamespace) const;

	/**
	 * Lookup naming tokens given a namespace. This will look first in cached naming tokens,
	 * then native classes. Call ClearCachedNamingTokens to reset the cache
	 * and force a full lookup.
	 *
	 * @param InNamespace The namespace of the tokens.
	 * @return The found Naming Tokens object, or nullptr.
	 */
	UFUNCTION(BlueprintCallable, Category = "NamingTokens")
	UE_API UNamingTokens* GetNamingTokensNative(const FString& InNamespace) const;

	/**
	 * Lookup multiple naming tokens from multiple namespaces.
	 *
	 * @param InNamespaces An array of all token namespaces.
	 * @return An array of matching tokens objects.
	 */
	UFUNCTION(BlueprintCallable, Category = "NamingTokens")
	UE_API TArray<UNamingTokens*> GetMultipleNamingTokens(const TArray<FString>& InNamespaces) const;

	/**
	 * Parse and evaluate token text.
	 * 
	 * @param InTokenText The text containing unprocessed tokens.
	 * @param InFilter [Optional] Filter to determine which namespaces to use.
	 * @param InContexts [Optional] Context objects to pass to naming tokens.
	 *
	 * @return The result of the evaluation.
	 */
	UFUNCTION(BlueprintCallable, Category = "NamingTokens", meta = (AutoCreateRefTerm = "InFilter,InContexts"))
	UE_API FNamingTokenResultData EvaluateTokenText(const FText& InTokenText, const FNamingTokenFilterArgs& InFilter, const TArray<UObject*>& InContexts);
	UE_API FNamingTokenResultData EvaluateTokenText(const FText& InTokenText, const FNamingTokenFilterArgs& InFilter = FNamingTokenFilterArgs());
	
	/**
	 * Parse and evaluate token string.
	 * 
	 * @param InTokenString The string containing unprocessed tokens.
	 * @param InFilter [Optional] Filter to determine which namespaces to use.
	 * @param InContexts [Optional] Context objects to pass to naming tokens.
	 *
	 * @return The result of the evaluation.
	 */
	UFUNCTION(BlueprintCallable, Category = "NamingTokens", meta = (AutoCreateRefTerm = "InFilter,InContexts"))
	UE_API FNamingTokenResultData EvaluateTokenString(const FString& InTokenString, const FNamingTokenFilterArgs& InFilter, const TArray<UObject*>& InContexts);
	UE_API FNamingTokenResultData EvaluateTokenString(const FString& InTokenString, const FNamingTokenFilterArgs& InFilter = FNamingTokenFilterArgs());

	/**
	 * Given a list of tokens, return a list of all found tokens and their values.
	 * 
	 * @param InTokenList A list of raw tokens. Tokens can include their namespace, but do not include brackets.
	 * @param InFilter [Optional] Filter to determine which namespaces to use.
	 * @param InContexts [Optional] Context objects to pass to naming tokens.
	 *
	 * @return Evaluation data for each token.
	 */
	UFUNCTION(BlueprintCallable, Category = "NamingTokens", meta = (AutoCreateRefTerm = "InFilter,InContexts"))
	UE_API TArray<FNamingTokenValueData> EvaluateTokenList(const TArray<FString>& InTokenList, const FNamingTokenFilterArgs& InFilter, const TArray<UObject*>& InContexts);
	UE_API TArray<FNamingTokenValueData> EvaluateTokenList(const TArray<FString>& InTokenList, const FNamingTokenFilterArgs& InFilter = FNamingTokenFilterArgs());
	
	/**
	 * Register tokens as a global namespace. This prevents the need to include the namespace in a token string.
	 * @param InNamespace The namespace of the tokens to register.
	 */
	UFUNCTION(BlueprintCallable, Category = "NamingTokens")
	UE_API void RegisterGlobalNamespace(const FString& InNamespace);

	/**
	 * Remove tokens from a global namespace.
	 * @param InNamespace The namespace of the tokens to unregister.
	 */
	UFUNCTION(BlueprintCallable, Category = "NamingTokens")
	UE_API void UnregisterGlobalNamespace(const FString& InNamespace);

	/** Checks if a namespace is registered globally. */
	UFUNCTION(BlueprintCallable, Category = "NamingTokens")
	UE_API bool IsGlobalNamespaceRegistered(const FString& InNamespace) const;
	
	/** Retrieve the registered global namespaces. */
	UFUNCTION(BlueprintCallable, Category = "NamingTokens")
	UE_API TArray<FString> GetGlobalNamespaces() const;

	/** Retrieve all discovered namespaces. */
	UFUNCTION(BlueprintCallable, Category = "NamingTokens")
	UE_API TArray<FString> GetAllNamespaces(bool bIncludePrivateNamespaces = false) const;

	/** Register a delegate to filter the list of namespaces that can be used to evaluate tokens */
	UE_API void RegisterNamespaceFilter(const FName OwnerName, FFilterNamespace Delegate);

	/** Remove a delegate from the list of namespace filters */
	UE_API void UnregisterNamespaceFilter(const FName OwnerName);

	/** Locate all reference naming token namespaces from a given string. */
	UE_API TSet<FString> GetNamingTokenNamespacesFromString(const FString& InTokenString, const FNamingTokenFilterArgs& InFilter = FNamingTokenFilterArgs()) const;
	
	/** Creates a friendly display string of all tokens. */
	UE_API FString GetFormattedTokensStringForDisplay(const FNamingTokenFilterArgs& InFilter) const;

	/**
	 * Clears all currently cached naming tokens. They will be loaded on demand when required.
	 * This can avoid an editor restart if a namespace is adjusted on a BP asset, and you don't want the old namespace to access it.
	 */
	UFUNCTION(BlueprintCallable, Category = "NamingTokens")
	UE_API void ClearCachedNamingTokens();

	/** If the cache is currently enabled. */
	UE_API bool IsCacheEnabled() const;

	/** Configure whether the cache is enabled or not. */
	UE_API void SetCacheEnabled(bool bEnabled);
	
private:
	/** Naming tokens currently loaded from assets. */
	UPROPERTY(Transient)
	mutable TMap<FString, TObjectPtr<UNamingTokens>> CachedNamingTokens;
	mutable FCriticalSection CachedNamingTokensMutex;

	UNamingTokens* LoadNamingToken(const TSoftClassPtr<UNamingTokens>& InTokensClass, const FString& InNamespace) const;
	UNamingTokens* GetNamingTokenFromCache(const FString& InNamespace, bool bNativeOnly = false) const;

	/** Namespaces considered global (don't need to include namespace to access). */
	TSet<FString> GlobalNamespaces;

	/** Filters that will execute just before evaluating a token string */
	TMap<FName, FFilterNamespace> FilterNamespaceDelegates;

	/** If the cache is currently enabled. */
	bool bIsCacheEnabled;
};

#undef UE_API
