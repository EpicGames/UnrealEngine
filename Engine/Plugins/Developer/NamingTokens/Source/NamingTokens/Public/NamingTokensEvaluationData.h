// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/DateTime.h"

#include "NamingTokensEvaluationData.generated.h"

/**
 * Shared data across token evaluation.
 */
USTRUCT(BlueprintType)
struct FNamingTokensEvaluationData
{
	GENERATED_BODY()

public:
	FNamingTokensEvaluationData();

	/** Initialize any constant data that isn't safe to do so in the ctor, like DateTime. */
	void Initialize();
	
	/** The current date time. */
	UPROPERTY(BlueprintReadOnly, Category = "NamingTokens", meta = (IgnoreForMemberInitializationTest))
	FDateTime CurrentDateTime;

	/** An optional array of context objects for this evaluation. */
	UPROPERTY(BlueprintReadOnly, Category = "NamingTokens")
	TArray<TObjectPtr<UObject>> Contexts;

	/** If case sensitivity should be enforced. */
	bool bForceCaseSensitive = false;
	
	/** If input data should be normalized first. */
	bool bNormalizeInput = true;
};