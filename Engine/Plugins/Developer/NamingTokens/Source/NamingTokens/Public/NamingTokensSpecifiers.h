// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::NamingTokens::Specifiers
{
	/** Specifier indicating naming tokens should be used. */
	const FString UseNamingTokens = TEXT("NamingTokens");
}