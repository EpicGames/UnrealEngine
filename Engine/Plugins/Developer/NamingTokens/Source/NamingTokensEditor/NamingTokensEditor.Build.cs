// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NamingTokensEditor : ModuleRules
{
    public NamingTokensEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "NamingTokens",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "AssetDefinition",
	            "BlueprintGraph",
	            "Blutility",
                "CoreUObject",
                "EditorToolEvents",
                "Engine",
                "EngineAssetDefinitions",
                "InputCore",
                "Slate",
                "SlateCore",
                "Projects",
                "UnrealEd",
            }
        );
    }
}