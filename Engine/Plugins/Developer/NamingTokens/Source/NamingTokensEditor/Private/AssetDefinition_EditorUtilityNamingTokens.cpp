// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_EditorUtilityNamingTokens.h"

#include "EditorUtilityNamingTokens.h"
#include "NamingTokensStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_EditorUtilityNamingTokens)

TSoftClassPtr<UObject> UAssetDefinition_EditorUtilityNamingTokens::GetAssetClass() const
{
	return UEditorUtilityNamingTokens::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_EditorUtilityNamingTokens::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::Misc) };
	return Categories;
}

FText UAssetDefinition_EditorUtilityNamingTokens::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetDefinition_EditorUtilityNamingTokens", "AssetDefinition_EditorUtilityNamingTokens_DisplayName", "Naming Tokens Editor Utility");
}

const FSlateBrush* UAssetDefinition_EditorUtilityNamingTokens::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return FNamingTokensStyle::Get().GetBrush("ClassThumbnail.NamingTokens");
}
