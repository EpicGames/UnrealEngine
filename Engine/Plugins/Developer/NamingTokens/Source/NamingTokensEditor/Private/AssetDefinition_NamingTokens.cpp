// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_NamingTokens.h"

#include "NamingTokens.h"
#include "NamingTokensStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_NamingTokens)

TSoftClassPtr<UObject> UAssetDefinition_NamingTokens::GetAssetClass() const
{
	return UNamingTokens::StaticClass();
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_NamingTokens::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] = { FAssetCategoryPath(EAssetCategoryPaths::Misc, NSLOCTEXT("AssetDefinition_NamingTokens", "NamingTokens_CategorySection", "Other"), ECategoryMenuType::Section) };
	return Categories;
}

FText UAssetDefinition_NamingTokens::GetAssetDisplayName() const
{
	return NSLOCTEXT("AssetDefinition_NamingTokens", "AssetDefinition_NamingTokens_DisplayName", "Naming Tokens");
}

const FSlateBrush* UAssetDefinition_NamingTokens::GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
{
	return FNamingTokensStyle::Get().GetBrush("ClassThumbnail.NamingTokens");
}
