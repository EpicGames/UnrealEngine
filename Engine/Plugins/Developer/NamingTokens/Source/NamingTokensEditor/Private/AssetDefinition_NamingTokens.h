// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Script/AssetDefinition_Blueprint.h"

#include "AssetDefinition_NamingTokens.generated.h"

UCLASS()
class UAssetDefinition_NamingTokens : public UAssetDefinition_Blueprint
{
	GENERATED_BODY()
	
public:
	virtual TSoftClassPtr<UObject> GetAssetClass() const override;
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override;
	virtual FText GetAssetDisplayName() const override;
	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const override;
};
