// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokensCustomization.h"

#include "NamingTokenData.h"
#include "NamingTokens.h"
#include "Utils/NamingTokensEditorUtils.h"
#include "Utils/NamingTokenUtils.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/Blueprint.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "IPropertyUtilities.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "NamingTokensCustomization"

namespace UE::NamingTokens::Customization::Private
{
	// If the error message should be visible.
	EVisibility GetErrorVisibilityFromProperty(const TSharedPtr<IPropertyHandle>& InPropertyHandle, const TSharedPtr<FText>& InErrorMessage)
	{
		if (InPropertyHandle.IsValid() && InErrorMessage.IsValid())
		{
			FString NamespaceValue;
			if (InPropertyHandle->GetValue(NamespaceValue) == FPropertyAccess::Result::Success
				&& Utils::ValidateName(NamespaceValue, *InErrorMessage))
			{
				return EVisibility::Collapsed;
			}
		}

		return EVisibility::Visible;
	}

	// Construct the error tooltip message.
	FText CreateErrorTooltipMessage(const TSharedPtr<FText>& InErrorMessage)
	{
		if (InErrorMessage.IsValid() && !InErrorMessage->IsEmpty())
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("Error"), *InErrorMessage);
			return FText::Format(LOCTEXT("ValueError", "Error: {Error}. Alphanumeric and '_' characters are allowed."), Args);
		}
		return FText::GetEmpty();
	}
	
	// Creates the standard row widget for a property, adding in an error icon for validation.
	void CreateRowWidgetWithError(IDetailPropertyRow& InRow, const TSharedPtr<IPropertyHandle>& InPropertyHandle,
		const TSharedPtr<FText>& InErrorMessage)
	{
		InRow.CustomWidget()
		.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.MaxDesiredWidth(250.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				InPropertyHandle->CreatePropertyValueWidget()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.Padding(2.f, 0.f, 0.f, 0.f)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Error"))
				.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.AccentRed"))
				.ToolTipText_Lambda([InErrorMessage]()
				{
					return CreateErrorTooltipMessage(InErrorMessage);
				})
				.Visibility_Lambda([InPropertyHandle, InErrorMessage]()
				{
					return GetErrorVisibilityFromProperty(InPropertyHandle, InErrorMessage);
				})
			]
		];
	}
}

void FNamingTokensCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	NamespacePropertyHandle = DetailBuilder.GetProperty(UNamingTokens::GetNamespacePropertyName());
	TokenKeyErrorMessage = MakeShared<FText>();
	if (NamespacePropertyHandle.IsValid())
	{
		if (IDetailPropertyRow* Row = DetailBuilder.EditDefaultProperty(NamespacePropertyHandle))
		{
			UE::NamingTokens::Customization::Private::CreateRowWidgetWithError(*Row, NamespacePropertyHandle, TokenKeyErrorMessage);
		}
	}
}

void FNamingTokensDataCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow,
                                                     IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	CustomizationUtilsPtr = &CustomizationUtils;
	NamespaceErrorMessage = MakeShared<FText>();
	
	if (const UNamingTokens* OwningTokens = GetOwningNamingTokens())
	{
		OwningBlueprint = UBlueprint::GetBlueprintFromClass(OwningTokens->GetClass());
		if (OwningBlueprint.IsValid())
		{
			FunctionNameHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNamingTokenData, FunctionName));
			check(FunctionNameHandle.IsValid());

			TokenKeyHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FNamingTokenData, TokenKey));
			check(TokenKeyHandle.IsValid());
		}
	}

	// If owning blueprint is null, then we may not be customizing the naming tokens directly, such as a settings object,
	// or the tokens are predefined and can't be extended in BP.
	
	HeaderRow
	.NameContent()
	[
		PropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		PropertyHandle->CreatePropertyValueWidget(/* bDisplayDefaultPropertyButtons (adds identical options) */ false)
	];
}

void FNamingTokensDataCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	if (!FunctionNameHandle.IsValid() || !TokenKeyHandle.IsValid())
	{
		return;
	}
	
	const bool bShouldApplyCustomization = OwningBlueprint.IsValid();

	if (bShouldApplyCustomization)
	{
		const TArray<UFunction*> Functions = GetAvailableFunctions();
		for (const UFunction* Function : Functions)
		{
			FunctionNames.Add(MakeShared<FString>(Function->GetName()));
		}

		FString CurrentValue;
		FunctionNameHandle->GetValue(CurrentValue);

		// Default selected function
		for (const TSharedPtr<FString>& Name : FunctionNames)
		{
			if (*Name == CurrentValue)
			{
				SelectedFunctionName = Name;
				break;
			}
		}

		FunctionNameHandle->MarkHiddenByCustomization();
	}
	
	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);
	
	for (uint32 ChildNum = 0; ChildNum < NumChildren; ++ChildNum)
	{
		const TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildNum);
		if (!ChildHandle.IsValid() || (bShouldApplyCustomization && ChildHandle->GetProperty() == FunctionNameHandle->GetProperty()))
		{
			continue;
		}
		IDetailPropertyRow& Row = ChildBuilder.AddProperty(ChildHandle.ToSharedRef());

		if (ChildHandle->GetProperty() == TokenKeyHandle->GetProperty())
		{
			UE::NamingTokens::Customization::Private::CreateRowWidgetWithError(Row, TokenKeyHandle, NamespaceErrorMessage);
		}
	}

	if (!bShouldApplyCustomization)
	{
		return;
	}
	
	FDetailWidgetRow& HeaderRow = ChildBuilder.AddProperty(FunctionNameHandle.ToSharedRef()).CustomWidget();
	HeaderRow.NameContent()
	[
		FunctionNameHandle->CreatePropertyNameWidget()
	];
	HeaderRow.ValueContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		[
			SNew(SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&FunctionNames)
			.OnGenerateWidget(this, &FNamingTokensDataCustomization::MakeComboBoxWidget)
			.OnSelectionChanged(this, &FNamingTokensDataCustomization::OnFunctionSelected, FunctionNameHandle)
			.InitiallySelectedItem(SelectedFunctionName)
			[
				SNew(STextBlock)
				.Text(this, &FNamingTokensDataCustomization::GetSelectedFunctionText)
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("PListEditor.Button_AddToArray"))
			]
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("AddFunction", "Create and assign a new function graph for evaluating tokens.\nThis requires a valid Token Key entered."))
			.OnClicked(this, &FNamingTokensDataCustomization::OnAddFunctionClicked)
			.IsEnabled(this, &FNamingTokensDataCustomization::CanAddFunction)
		]
	];
}

UNamingTokens* FNamingTokensDataCustomization::GetOwningNamingTokens() const
{
	if (CustomizationUtilsPtr)
	{
		const TSharedPtr<IPropertyUtilities> PropertyUtilities = CustomizationUtilsPtr->GetPropertyUtilities();
		if (PropertyUtilities.IsValid())
		{
			const TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized = PropertyUtilities->GetSelectedObjects();

			if (ObjectsBeingCustomized.Num() == 1)
			{
				if (UNamingTokens* Tokens = Cast<UNamingTokens>(ObjectsBeingCustomized[0]))
				{
					return Tokens;
				}
			}
		}
	}
	return nullptr;
}

TArray<UFunction*> FNamingTokensDataCustomization::GetAvailableFunctions() const
{
	TArray<UFunction*> Result;
	if (OwningBlueprint.IsValid())
	{
		for (TFieldIterator<UFunction> It(OwningBlueprint->SkeletonGeneratedClass ?
			OwningBlueprint->SkeletonGeneratedClass : OwningBlueprint->GeneratedClass,
			EFieldIteratorFlags::IncludeSuper); It; ++It)
		{
			if (UE::NamingTokens::Utils::ValidateTokenFunction(*It))
			{
				Result.Add(*It);
			}
		}
	}

	return Result;
}

TSharedRef<SWidget> FNamingTokensDataCustomization::MakeComboBoxWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock)
		.Text(FText::FromString(*InItem));
}

void FNamingTokensDataCustomization::OnFunctionSelected(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo,
	TSharedPtr<IPropertyHandle> InFunctionNameHandle)
{
	if (NewValue.IsValid())
	{
		SelectedFunctionName = NewValue;
		InFunctionNameHandle->SetValue(*NewValue);
	}
}

FText FNamingTokensDataCustomization::GetSelectedFunctionText() const
{
	return SelectedFunctionName.IsValid() ? FText::FromString(*SelectedFunctionName) : FText::FromString(TEXT("Select Function"));
}

FReply FNamingTokensDataCustomization::OnAddFunctionClicked()
{
	if (UBlueprint* Blueprint = OwningBlueprint.Get())
	{
		FString TokenKey;
		TokenKeyHandle->GetValue(TokenKey);
		FScopedTransaction Transaction(LOCTEXT("CreateTokenGraph", "Create Token Graph"));

		const FName NewFunctionName = UE::NamingTokens::Utils::Editor::Private::CreateNewTokenGraph(Blueprint, TokenKey);
		FunctionNameHandle->SetValue(NewFunctionName);
	}
	
	return FReply::Handled();
}

bool FNamingTokensDataCustomization::CanAddFunction() const
{
	FString TokenKey;
	if (TokenKeyHandle.IsValid())
	{
		TokenKeyHandle->GetValue(TokenKey);

		FText ErrorMessage;
		if (!UE::NamingTokens::Utils::ValidateName(TokenKey, ErrorMessage))
		{
			return false;
		}
			
		const FString BaseFunctionName = UE::NamingTokens::Utils::Editor::Private::CreateBaseTokenFunctionName(TokenKey);

		// Make sure the name isn't already being used.
		for (const TSharedPtr<FString>& FunctionName : FunctionNames)
		{
			if (FunctionName.IsValid() && **FunctionName == BaseFunctionName)
			{
				return false;
			}
		}
	}
	
	return !TokenKey.IsEmpty();
}

#undef LOCTEXT_NAMESPACE
