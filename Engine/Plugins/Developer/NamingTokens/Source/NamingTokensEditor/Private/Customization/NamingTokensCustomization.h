// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "IPropertyTypeCustomization.h"
#include "Layout/Visibility.h"

class UNamingTokens;
class SWidget;

class FNamingTokensCustomization final : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FNamingTokensCustomization);
	}

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	
private:
	/** Handle to namespace property. */
	TSharedPtr<IPropertyHandle> NamespacePropertyHandle;
	/** Stores error text. */
	TSharedPtr<FText> TokenKeyErrorMessage;
};

class FNamingTokensDataCustomization final : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FNamingTokensDataCustomization());
	}

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils) override;

	/** The naming tokens owning this customization. */
	UNamingTokens* GetOwningNamingTokens() const;

private:
	/** Available functions which can be assigned to naming tokens. */
	TArray<UFunction*> GetAvailableFunctions() const;
	/** Make the combo box for selecting functions. */
	TSharedRef<SWidget> MakeComboBoxWidget(TSharedPtr<FString> InItem);
	/** When the user selects a function. */
	void OnFunctionSelected(
		TSharedPtr<FString> NewValue, 
		ESelectInfo::Type SelectInfo, 
		TSharedPtr<IPropertyHandle> InFunctionNameHandle);
	/** The text of the currently selected function. */
	FText GetSelectedFunctionText() const;
	/** User clicked the button to add a new function. */
	FReply OnAddFunctionClicked();
	/** If a new function can be added. */
	bool CanAddFunction() const;

private:
	/** Customization pointer. */
	IPropertyTypeCustomizationUtils* CustomizationUtilsPtr = nullptr;
	/** Blueprint of the naming tokens. */
	TWeakObjectPtr<UBlueprint> OwningBlueprint;

	/** All available function names. */
	TArray<TSharedPtr<FString>> FunctionNames;
	/** The selected function name. */
	TSharedPtr<FString> SelectedFunctionName;
	/** Property handle to the function name. */
	TSharedPtr<IPropertyHandle> FunctionNameHandle;
	/** Property handle to the token key. */
	TSharedPtr<IPropertyHandle> TokenKeyHandle;
	/** Stores error text. */
	TSharedPtr<FText> NamespaceErrorMessage;
};