// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityNamingTokens.h"

#include "EditorToolDelegates.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityNamingTokens)

UEditorUtilityNamingTokens::UEditorUtilityNamingTokens()
{
}

void UEditorUtilityNamingTokens::PostLoad()
{
	Super::PostLoad();
	// UEditorUtilityObject::PostLoad() calls this, but we don't extend UEditorUtilityObject, so we call this for consistency.
	FEditorToolDelegates::OnEditorToolStarted.Broadcast(GetClass()->GetName());
}

bool UEditorUtilityNamingTokens::IsEditorOnly() const
{
	return true;
}
