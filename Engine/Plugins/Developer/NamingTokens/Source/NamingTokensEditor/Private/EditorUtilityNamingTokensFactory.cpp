// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorUtilityNamingTokensFactory.h"

#include "EditorUtilityBlueprint.h"
#include "EditorUtilityNamingTokens.h"

#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Kismet2/KismetEditorUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditorUtilityNamingTokensFactory)

UEditorUtilityNamingTokensFactory::UEditorUtilityNamingTokensFactory()
{
	SupportedClass = UEditorUtilityNamingTokens::StaticClass();

	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UEditorUtilityNamingTokensFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UClass* ParentClass = SupportedClass;
	check(InClass->IsChildOf(ParentClass));

	UBlueprint* NewBP = FKismetEditorUtilities::CreateBlueprint(
		ParentClass,
		InParent,
		InName,
		BPTYPE_Normal,
		UEditorUtilityBlueprint::StaticClass(),
		UBlueprintGeneratedClass::StaticClass()
	);

	return NewBP;
}
