// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"

#include "EditorUtilityNamingTokensFactory.generated.h"

/** Factory class used to create new UEditorUtilityNamingTokens objects */
UCLASS(MinimalAPI)
class UEditorUtilityNamingTokensFactory : public UFactory
{
	GENERATED_BODY()

public:
	UEditorUtilityNamingTokensFactory();

	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};
