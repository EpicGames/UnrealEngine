// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokensEditorModule.h"

#include "Customization/NamingTokensCustomization.h"
#include "NamingTokens.h"
#include "NamingTokensStyle.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "FNamingTokensEditorModule"

void FNamingTokensEditorModule::StartupModule()
{
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomClassLayout(
		UNamingTokens::StaticClass()->GetFName(),
		FOnGetDetailCustomizationInstance::CreateStatic(&FNamingTokensCustomization::MakeInstance)
	);

	PropertyModule.RegisterCustomPropertyTypeLayout(FNamingTokenData::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNamingTokensDataCustomization::MakeInstance));

	FNamingTokensStyle::Get();
}

void FNamingTokensEditorModule::ShutdownModule()
{
	if (UObjectInitialized() && !IsEngineExitRequested())
	{
		if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			PropertyEditorModule->UnregisterCustomClassLayout(UNamingTokens::StaticClass()->GetFName());
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(FNamingTokenData::StaticStruct()->GetFName());
		}
	}
}

#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(FNamingTokensEditorModule, NamingTokensEditor)