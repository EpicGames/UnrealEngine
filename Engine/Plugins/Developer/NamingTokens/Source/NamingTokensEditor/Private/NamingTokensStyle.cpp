// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokensStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateTypes.h"

FName FNamingTokensStyle::StyleName("NamingTokensStyle");

FNamingTokensStyle& FNamingTokensStyle::Get()
{
	static FNamingTokensStyle Inst;
	return Inst;
}

FNamingTokensStyle::FNamingTokensStyle()
	: FSlateStyleSet(StyleName)
{
	// Icons
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("NamingTokens"))->GetContentDir();
	SetContentRoot(ContentDir);

	Set("ClassIcon.NamingTokens", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/Token_16.svg")), Icon16x16));
	Set("ClassThumbnail.NamingTokens", new FSlateVectorImageBrush(RootToContentDir(TEXT("Slate/Token_64.svg")), Icon64x64));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FNamingTokensStyle::~FNamingTokensStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
