// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokensEditorUtils.h"

#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Utils/NamingTokenUtils.h"

FString UE::NamingTokens::Utils::Editor::Private::CreateBaseTokenFunctionName(const FString& InTokenKey)
{
	return TEXT("ProcessToken_") + InTokenKey;
}

FName UE::NamingTokens::Utils::Editor::Private::CreateNewTokenGraph(UBlueprint* InBlueprint, const FString& InTokenKey)
{
	check(InBlueprint);
	
	const FString BaseName = CreateBaseTokenFunctionName(InTokenKey);
	const FName FunctionName = FBlueprintEditorUtils::GenerateUniqueGraphName(InBlueprint, BaseName);

	InBlueprint->Modify();
		
	UEdGraph* NewGraph = FBlueprintEditorUtils::CreateNewGraph(InBlueprint, FunctionName, UEdGraph::StaticClass(), UEdGraphSchema_K2::StaticClass());
	
	UFunction* FunctionSignature = UE::NamingTokens::Utils::GetProcessTokenFunctionSignature();
		
	FBlueprintEditorUtils::AddFunctionGraph<UFunction>(InBlueprint, NewGraph, /*bIsUserCreated=*/ true, FunctionSignature);
	FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(NewGraph);

	return FunctionName;
}
