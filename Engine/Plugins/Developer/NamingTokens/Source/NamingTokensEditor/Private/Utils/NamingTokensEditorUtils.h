// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UBlueprint;

namespace UE::NamingTokens::Utils::Editor::Private
{
	/** Create the default function token name to use, before unique naming adjustments. */
	FString CreateBaseTokenFunctionName(const FString& InTokenKey);
	
	/** Creates a new k2 graph in a blueprint for a given token key. */
	FName CreateNewTokenGraph(UBlueprint* InBlueprint, const FString& InTokenKey);
}
