// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NamingTokens.h"

#include "EditorUtilityNamingTokens.generated.h"

#define UE_API NAMINGTOKENSEDITOR_API

/**
 * Subclass to define naming tokens for editor use only.
 */
UCLASS(MinimalAPI, Blueprintable, Abstract, meta = (ShowWorldContextPin))
class UEditorUtilityNamingTokens : public UNamingTokens
{
	GENERATED_BODY()

public:
	UE_API UEditorUtilityNamingTokens();

	UE_API virtual void PostLoad() override;

	UE_API virtual bool IsEditorOnly() const override;
};

#undef UE_API
