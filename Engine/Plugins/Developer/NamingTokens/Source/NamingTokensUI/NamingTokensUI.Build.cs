// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NamingTokensUI : ModuleRules
{
    public NamingTokensUI(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
                "CoreUObject",
                "NamingTokens",
                "UMG"
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Engine",
                "InputCore",
                "Slate",
                "SlateCore"
            }
        );
        
        if (Target.Type == TargetType.Editor)
        {
	        PublicDependencyModuleNames.AddRange(
		        new string[] {
			        "UnrealEd",
		        }
	        );
        }
    }
}