// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokensEditableText.h"

#include "NamingTokensStringSyntaxHighlighterMarshaller.h"
#include "SNamingTokensEditableTextBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NamingTokensEditableText)

UNamingTokensEditableText::UNamingTokensEditableText()
{
	ArgumentStyle = FNamingTokensStringSyntaxHighlighterMarshaller::GetDefaultArgumentStyle();
	BackgroundColor = FLinearColor(0, 0, 0, 0);
}

const FText& UNamingTokensEditableText::GetResolvedText() const
{
	check(MyNamingTokensEditableText.IsValid());
	return MyNamingTokensEditableText->GetResolvedText();
}

const FText& UNamingTokensEditableText::GetTokenizedText() const
{
	check(MyNamingTokensEditableText.IsValid());
	return MyNamingTokensEditableText->GetTokenizedText();
}

void UNamingTokensEditableText::SetWidgetArgumentStyle(const FTextBlockStyle& InWidgetStyle)
{
	ArgumentStyle = InWidgetStyle;
	if (MyNamingTokensEditableText.IsValid())
	{
		MyNamingTokensEditableText->SetArgumentStyle(&InWidgetStyle);
	}
}

void UNamingTokensEditableText::SetBackgroundColor(const FSlateColor& InColor)
{
	BackgroundColor = InColor;
	if (MyNamingTokensEditableText.IsValid() && EditableTextBoxStyle.IsValid())
	{
		EditableTextBoxStyle->SetBackgroundColor(InColor);
		MyNamingTokensEditableText->SetNormalStyle(EditableTextBoxStyle.Get());
	}
}

void UNamingTokensEditableText::SetContexts(const TArray<UObject*>& InContexts)
{
	Contexts = InContexts;
}

void UNamingTokensEditableText::SetDisplayTokenIcon(bool bValue)
{
	bDisplayTokenIcon = bValue;
}

void UNamingTokensEditableText::SetDisplayErrorMessage(bool bValue)
{
	bDisplayErrorMessage = bValue;
}

void UNamingTokensEditableText::SetDisplayBorderImage(bool bValue)
{
	bDisplayBorderImage = bValue;
}

bool UNamingTokensEditableText::GetDisplayTokenIcon() const
{
	return bDisplayTokenIcon;
}

bool UNamingTokensEditableText::GetDisplayErrorMessage() const
{
	return bDisplayErrorMessage;
}

bool UNamingTokensEditableText::GetDisplayBorderImage() const
{
	return bDisplayBorderImage;
}

void UNamingTokensEditableText::SetCanDisplayResolvedText(bool bValue)
{
	bCanDisplayResolvedText = bValue;
}

bool UNamingTokensEditableText::GetCanDisplayResolvedText() const
{
	return bCanDisplayResolvedText;
}

void UNamingTokensEditableText::SetShowUnknownTokenWarning(bool bValue)
{
	bShowUnknownTokenWarning = bValue;
	if (MyNamingTokensEditableText.IsValid())
	{
		MyNamingTokensEditableText->SetShowUnknownTokenWarning(bValue);
	}
}

bool UNamingTokensEditableText::GetShowUnknownTokenWarning() const
{
	return bShowUnknownTokenWarning;
}

void UNamingTokensEditableText::SetShowUnsetTokenWarning(bool bValue)
{
	bShowUnsetTokenWarning = bValue;
	if (MyNamingTokensEditableText.IsValid())
	{
		MyNamingTokensEditableText->SetShowUnsetTokenWarning(bValue);
	}
}

bool UNamingTokensEditableText::GetShowUnsetTokenWarning() const
{
	return bShowUnsetTokenWarning;
}

#if WITH_EDITOR
void UNamingTokensEditableText::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UNamingTokensEditableText, Text)
		|| PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UNamingTokensEditableText, bDisplayResolvedTextInDesigner))
	{
		if (MyNamingTokensEditableText.IsValid())
		{
			MyNamingTokensEditableText->SetTokenizedText(Text);
		}
		SynchronizeProperties();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif

TSharedRef<SWidget> UNamingTokensEditableText::RebuildWidget()
{
	// We only use text box style, but slate needs editable text box style.
	EditableTextBoxStyle = MakeShared<FEditableTextBoxStyle>(FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"));
	EditableTextBoxStyle->SetTextStyle(WidgetStyle);
	EditableTextBoxStyle->SetBackgroundColor(BackgroundColor);
	
	// We broadcast a BP event that may be setting data we need, and our widget is being constructed or reconstructed.
	// An evaluation generally occurs on instantiation, so we fire one off without the widget to account for this case.
	{
		MyNamingTokensEditableText.Reset();
		OnPreEvaluateNamingTokensCallback();
	}

	MyNamingTokensEditableText = SNew(SNamingTokensEditableTextBox)
		.Text(GetText())
		.IsReadOnly(GetIsReadOnly())
		.CanDisplayResolvedText(MakeAttributeUObject(this, &UNamingTokensEditableText::CanDisplayResolvedTextAttribute))
		.AllowMultiLine(bIsMultiline)
		.Style(&*EditableTextBoxStyle)
		.ArgumentStyle(&ArgumentStyle)
		.FilterArgs(FilterArgs)
		.Contexts(Contexts)
		.EnableSuggestionDropdown(bEnableSuggestionDropdown)
		.NamespaceSuggestionPriority(NamespaceSuggestionPriority)
		.DisplayTokenIcon_UObject(this, &UNamingTokensEditableText::GetDisplayTokenIcon)
		.DisplayErrorMessage_UObject(this, &UNamingTokensEditableText::GetDisplayErrorMessage)
		.DisplayBorderImage_UObject(this, &UNamingTokensEditableText::GetDisplayBorderImage)
		.ShowUnknownTokenWarning(bShowUnknownTokenWarning)
		.ShowUnsetTokenWarning(bShowUnsetTokenWarning)
		.OnPreEvaluateNamingTokens_UObject(this, &UNamingTokensEditableText::OnPreEvaluateNamingTokensCallback);

	// Make sure our primary editable text (managed by parent) points to the editable text we create from the naming tokens slate text box.
	MyMultiLineEditableText	= MyNamingTokensEditableText->GetEditableText();

	return MyNamingTokensEditableText.ToSharedRef();
}

void UNamingTokensEditableText::SynchronizeProperties()
{
	// Skip parent call, we want to avoid setting the text directly and use our bound text attribute instead.
	UWidget::SynchronizeProperties();

	if (!MyMultiLineEditableText.IsValid())
	{
		return;
	}
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TAttribute<FText> HintTextBinding = PROPERTY_BINDING(FText, HintText);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	EditableTextBoxStyle->SetTextStyle(WidgetStyle);
	MyNamingTokensEditableText->SetNormalStyle(&*EditableTextBoxStyle);
	MyNamingTokensEditableText->SetArgumentStyle(&ArgumentStyle);
	MyNamingTokensEditableText->SetEnableSuggestionDropdown(bEnableSuggestionDropdown);
	MyNamingTokensEditableText->SetFullyQualifyFilteredNamespaces(bFullyQualifyFilteredNamespaces);
	MyNamingTokensEditableText->SetNamespaceSuggestionPriority(NamespaceSuggestionPriority);
	MyNamingTokensEditableText->SetShowUnknownTokenWarning(bShowUnknownTokenWarning);
	MyNamingTokensEditableText->SetShowUnsetTokenWarning(bShowUnsetTokenWarning);
	
	MyMultiLineEditableText->SetText(MyNamingTokensEditableText->GetBoundText());
	MyMultiLineEditableText->SetHintText(HintTextBinding);
	MyMultiLineEditableText->SetAllowContextMenu(AllowContextMenu);
	MyMultiLineEditableText->SetIsReadOnly(GetIsReadOnly());
	MyMultiLineEditableText->SetVirtualKeyboardDismissAction(VirtualKeyboardDismissAction);
	MyMultiLineEditableText->SetSelectAllTextWhenFocused(GetSelectAllTextWhenFocused());
	MyMultiLineEditableText->SetClearTextSelectionOnFocusLoss(GetClearTextSelectionOnFocusLoss());
	MyMultiLineEditableText->SetRevertTextOnEscape(GetRevertTextOnEscape());
	MyMultiLineEditableText->SetClearKeyboardFocusOnCommit(GetClearKeyboardFocusOnCommit());

	Super::SynchronizeTextLayoutProperties(*MyMultiLineEditableText);
}

void UNamingTokensEditableText::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);
	MyNamingTokensEditableText.Reset();
}

bool UNamingTokensEditableText::CanDisplayResolvedTextAttribute() const
{
	if (GetCanDisplayResolvedText())
	{
#if WITH_EDITOR
		if (IsDesignTime())
		{
			return bDisplayResolvedTextInDesigner;
		}
#endif
	
		return true;
	}
	
	return false;
}

void UNamingTokensEditableText::OnPreEvaluateNamingTokensCallback()
{
	OnPreEvaluateNamingTokens.Broadcast();
	
	if (MyNamingTokensEditableText.IsValid())
	{
		MyNamingTokensEditableText->SetFilterArgs(FilterArgs);
		MyNamingTokensEditableText->SetContexts(Contexts);
	}
}
