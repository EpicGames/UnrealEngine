// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokensStringSyntaxHighlighterMarshaller.h"

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Framework/Text/IRun.h"
#include "Framework/Text/ISlateRun.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Text/TextLayout.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StarshipCoreStyle.h"
#include "Templates/UnrealTemplate.h"

constexpr static const TCHAR* Operators[] = {
	TEXT("{"),
	TEXT("}")
};

TSharedRef<FNamingTokensStringSyntaxHighlighterMarshaller> FNamingTokensStringSyntaxHighlighterMarshaller::Create(const FSyntaxTextStyle& InSyntaxTextStyle)
{
	TArray<FSyntaxTokenizer::FRule> TokenizerRules;

	// operators
	for(const TCHAR* Operator : Operators)
	{
		TokenizerRules.Emplace(FSyntaxTokenizer::FRule(Operator));
	}	

	return MakeShared<FNamingTokensStringSyntaxHighlighterMarshaller>(FSyntaxTokenizer::Create(TokenizerRules), InSyntaxTextStyle);
}

const FTextBlockStyle& FNamingTokensStringSyntaxHighlighterMarshaller::GetDefaultNormalStyle()
{
	static FTextBlockStyle DefaultNormalStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>(TEXT("NormalText"));
	return DefaultNormalStyle;
}

const FTextBlockStyle& FNamingTokensStringSyntaxHighlighterMarshaller::GetDefaultArgumentStyle()
{
	static FTextBlockStyle DefaultArgumentStyle = FAppStyle::GetWidgetStyle<FTextBlockStyle>(TEXT("NormalText"));
	DefaultArgumentStyle.SetFont(DEFAULT_FONT("Bold", FStarshipCoreStyle::RegularTextSize))
	.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f));
	
	return DefaultArgumentStyle;
}

void FNamingTokensStringSyntaxHighlighterMarshaller::ParseTokens(const FString& SourceString, FTextLayout& TargetTextLayout, TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines)
{
	enum class EParseState : uint8
	{
		None,
		LookingForArgument,
	};

	TArray<FTextLayout::FNewLineData> LinesToAdd;
	LinesToAdd.Reserve(TokenizedLines.Num());

	// Parse the tokens, generating the styled runs for each line
	EParseState ParseState = EParseState::None;
	for(const ISyntaxTokenizer::FTokenizedLine& TokenizedLine : TokenizedLines)
	{
		TSharedRef<FString> ModelString = MakeShared<FString>();
		TArray<TSharedRef<IRun>> Runs;

		for(const ISyntaxTokenizer::FToken& Token : TokenizedLine.Tokens)
		{
			const FStringView TokenText(*SourceString + Token.Range.BeginIndex, Token.Range.Len());

			const FTextRange ModelRange(ModelString->Len(), ModelString->Len() + TokenText.Len());
			ModelString->Append(TokenText);

			FRunInfo RunInfo(TEXT("SyntaxHighlight.Template.Normal"));
			FTextBlockStyle TextBlockStyle = SyntaxTextStyle.NormalTextStyle;

			bool bHasMatchedSyntax = false;
			if (Token.Type == ISyntaxTokenizer::ETokenType::Syntax)
			{
				if(ParseState == EParseState::None && TokenText == TEXT("{"))
				{
					RunInfo.Name = TEXT("SyntaxHighlight.Template.Argument");
					TextBlockStyle = SyntaxTextStyle.ArgumentTextStyle;
					ParseState = EParseState::LookingForArgument;
					bHasMatchedSyntax = true;
				}
				else if(ParseState == EParseState::LookingForArgument && TokenText == TEXT("}"))
				{
					RunInfo.Name = TEXT("SyntaxHighlight.Template.Normal");
					TextBlockStyle = SyntaxTextStyle.ArgumentTextStyle; // Closing brace should still use this style
					ParseState = EParseState::None;
				}
			}
			
			// It's possible that we fail to match a syntax token if we're in a state where it isn't parsed
			// In this case, we treat it as a literal token
			if (Token.Type == ISyntaxTokenizer::ETokenType::Literal || !bHasMatchedSyntax)
			{
				if(ParseState == EParseState::LookingForArgument)
				{
					RunInfo.Name = TEXT("SyntaxHighlight.Template.Argument");
					TextBlockStyle = SyntaxTextStyle.ArgumentTextStyle;
				}
			}

			TSharedRef< ISlateRun > Run = FSlateTextRun::Create(RunInfo, ModelString, TextBlockStyle, ModelRange);
			Runs.Add(Run);
		}

		LinesToAdd.Emplace(MoveTemp(ModelString), MoveTemp(Runs));
	}

	TargetTextLayout.AddLines(LinesToAdd);
}

FNamingTokensStringSyntaxHighlighterMarshaller::FNamingTokensStringSyntaxHighlighterMarshaller(TSharedPtr< ISyntaxTokenizer > InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle)
	: FSyntaxHighlighterTextLayoutMarshaller(MoveTemp(InTokenizer))
	, SyntaxTextStyle(InSyntaxTextStyle)
{
}
