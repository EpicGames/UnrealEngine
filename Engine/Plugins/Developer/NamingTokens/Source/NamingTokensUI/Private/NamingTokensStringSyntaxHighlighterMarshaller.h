// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Framework/Text/SyntaxHighlighterTextLayoutMarshaller.h"
#include "Framework/Text/SyntaxTokenizer.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"

class FTextLayout;

/**
 * Get/set the raw text to/from a text layout, and also inject syntax highlighting for our rich-text markup
 */
class FNamingTokensStringSyntaxHighlighterMarshaller : public FSyntaxHighlighterTextLayoutMarshaller
{
public:
	struct FSyntaxTextStyle
	{
		FSyntaxTextStyle()
			: NormalTextStyle(GetDefaultNormalStyle())
			, ArgumentTextStyle(GetDefaultArgumentStyle())
		{
		}
		
		FSyntaxTextStyle(const FTextBlockStyle& InNormalTextStyle, const FTextBlockStyle& InArgumentTextStyle)
			: NormalTextStyle(InNormalTextStyle)
			, ArgumentTextStyle(InArgumentTextStyle)
		{
		}
		
		FTextBlockStyle NormalTextStyle;
		FTextBlockStyle ArgumentTextStyle;
	};

	static TSharedRef<FNamingTokensStringSyntaxHighlighterMarshaller> Create(const FSyntaxTextStyle& InSyntaxTextStyle);

	virtual ~FNamingTokensStringSyntaxHighlighterMarshaller() override = default;

	/** Set the normal style of our syntax. */
	void SetNormalStyle(const FTextBlockStyle& InStyle)
	{
		SyntaxTextStyle.NormalTextStyle = InStyle;
	}

	/** Set the argument style for our syntax. */
	void SetArgumentStyle(const FTextBlockStyle& InStyle)
	{
		SyntaxTextStyle.ArgumentTextStyle = InStyle;
	}

	/** Retrieve the default normal style used. */
	static const FTextBlockStyle& GetDefaultNormalStyle();
	
	/** Retrieve the default argument style used. */
	static const FTextBlockStyle& GetDefaultArgumentStyle();
	
protected:
	// Allows MakeShared with private constructor
	friend class SharedPointerInternals::TIntrusiveReferenceController<FNamingTokensStringSyntaxHighlighterMarshaller, ESPMode::ThreadSafe>;
	
	virtual void ParseTokens(const FString& SourceString, FTextLayout& TargetTextLayout, TArray<ISyntaxTokenizer::FTokenizedLine> TokenizedLines) override;

	FNamingTokensStringSyntaxHighlighterMarshaller(TSharedPtr<ISyntaxTokenizer> InTokenizer, const FSyntaxTextStyle& InSyntaxTextStyle);

	/** Styles used to display the text */
	FSyntaxTextStyle SyntaxTextStyle;
};
