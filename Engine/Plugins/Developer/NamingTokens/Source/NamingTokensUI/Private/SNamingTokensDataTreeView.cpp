// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNamingTokensDataTreeView.h"

#include "NamingTokens.h"
#include "NamingTokensEngineSubsystem.h"

#include "Engine/Engine.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "SNamingTokenDataTreeView"

void SNamingTokenDataTreeView::Construct(const FArguments& InArgs, const TSharedPtr<SNamingTokenDataTreeViewWidget>& InTreeViewWidget)
{
	STreeView::Construct(InArgs);
	OwningTreeViewWidget = InTreeViewWidget;
}

void SNamingTokenDataTreeView::SetExpansionStateFromItems(const TArray<FNamingTokenDataTreeItemPtr>& InTreeItems)
{
	for (const FNamingTokenDataTreeItemPtr& TreeItem: InTreeItems)
	{
		SetItemExpansion(TreeItem, TreeItem->ShouldItemBeExpanded());
		SetExpansionStateFromItems(TreeItem->ChildItems);
	}
}

FReply SNamingTokenDataTreeView::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	check(OwningTreeViewWidget.IsValid());
	OwningTreeViewWidget.Pin()->NotifyFocusReceived();
	return STreeView<FNamingTokenDataTreeItemPtr>::OnFocusReceived(MyGeometry, InFocusEvent);
}

void SNamingTokenDataTreeViewRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
{
	Item = InArgs._Item;

	const FSuperRowType::FArguments SuperArgs;
	SMultiColumnTableRow::Construct(SuperArgs, InOwnerTable);
}

TSharedRef<SWidget> SNamingTokenDataTreeViewRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	const FNamingTokenDataTreeItemPtr ItemPtr = Item.Pin();
	if (ItemPtr.IsValid())
	{
		if (const TSharedPtr<SNamingTokenDataTreeView> TreeViewPtr = StaticCastSharedPtr<SNamingTokenDataTreeView>(OwnerTablePtr.Pin()))
		{
			const TWeakPtr<SNamingTokenDataTreeViewWidget> TreeViewWidgetWeakPtr = TreeViewPtr->GetOwningTreeViewWidget();
			if (const TSharedPtr<SNamingTokenDataTreeViewWidget> TreeViewWidgetPin = TreeViewWidgetWeakPtr.Pin())
			{
				const TSharedPtr<SBorder> Border = SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("NoBorder"));
		
				if (ItemPtr->NamingTokenData.IsValid() && ColumnName == SNamingTokenDataTreeViewWidget::PrimaryColumnName())
				{
					const FString FilterString = TreeViewWidgetPin->GetTokenFilterText();
					const FText FilterText = ItemPtr->NamingTokenData->TokenKey.StartsWith(FilterString)
						? FText::FromString(FilterString) : FText::GetEmpty();
					
					Border->SetContent(
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(16.f, 2.f)
						[
							SNew(SEditableTextBox)
							.Style(&GetCustomEditableTextBoxStyle())
							.Visibility(EVisibility::HitTestInvisible)
							.IsReadOnly(true)
							.Padding(0.f)
							.BackgroundColor(FLinearColor::Transparent)
							.ForegroundColor(FSlateColor::UseForeground())
							.ReadOnlyForegroundColor(FSlateColor::UseForeground())
							.Text(FText::FromString(ItemPtr->NamingTokenData->TokenKey))
							.SearchText(FilterText)
						]);
				}
				else if (!ItemPtr->Namespace.IsEmpty())
				{
					const FText NamespaceDisplayName = !ItemPtr->NamespaceDisplayName.IsEmpty() ?
						FText::Format(LOCTEXT("NamespaceDisplayName", "{0} ({1}:)"),
						ItemPtr->NamespaceDisplayName, FText::FromString(ItemPtr->Namespace))
							: FText::FromString(ItemPtr->Namespace);
			
					Border->SetContent(SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0.f, 0.f, 0.f, 0.f)
						.VAlign(VAlign_Center)
						[
							SNew(SExpanderArrow, SharedThis(this))
							.StyleSet(&FAppStyle::Get())
							.IndentAmount(5)
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Font(FCoreStyle::Get().GetFontStyle("ExpandableArea.TitleFont"))
							.Text(NamespaceDisplayName)
						]);
				}

				return Border.ToSharedRef();
			}
		}
	}

	return SNullWidget::NullWidget;
}

const FEditableTextBoxStyle& SNamingTokenDataTreeViewRow::GetCustomEditableTextBoxStyle()
{
	static FEditableTextBoxStyle EditableTextBoxStyle = []()
		{
			static FEditableTextBoxStyle Style = FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");
			Style.BackgroundImageReadOnly = *FCoreStyle::Get().GetBrush("NoBorder");

			return Style;
		}();
	return EditableTextBoxStyle;
}

SNamingTokenDataTreeViewWidget::~SNamingTokenDataTreeViewWidget()
{
}

void SNamingTokenDataTreeViewWidget::Construct(const FArguments& InArgs)
{
	ItemSelectionChangedDelegate = InArgs._OnSelectionChanged;
	ItemSelectedDelegate = InArgs._OnItemSelected;
	OnFocusedDelegate = InArgs._OnFocused;
	NamespacePriority = InArgs._NamespacePriority;
	
	TSharedPtr<SHeaderRow> HeaderRow;
	// Setup the columns.
	{
		SAssignNew(HeaderRow, SHeaderRow);

		SHeaderRow::FColumn::FArguments ColumnArgs;
		ColumnArgs.ColumnId(PrimaryColumnName());
		ColumnArgs.DefaultLabel(LOCTEXT("ItemLabel_HeaderText", "Primary"));

		// Don't draw the column header.
		HeaderRow->SetVisibility(EVisibility::Collapsed);
		HeaderRow->AddColumn(ColumnArgs);
	}

	PopulateTreeItems(InArgs._FilterArgs);

	ChildSlot
	[
		SNew(SBox)
		.MaxDesiredHeight(250.f)
		.Padding(1.f)
		[
			SAssignNew(TreeView, SNamingTokenDataTreeView, SharedThis(this))
			.SelectionMode(ESelectionMode::Single)
			.TreeItemsSource(&FilteredRootTreeItems)
			.HeaderRow(HeaderRow)
			.OnGenerateRow(this, &SNamingTokenDataTreeViewWidget::OnGenerateRowForTree)
			.OnGetChildren(this, &SNamingTokenDataTreeViewWidget::OnGetChildrenForTree)
			.OnExpansionChanged(this, &SNamingTokenDataTreeViewWidget::OnItemExpansionChanged)
			.OnSelectionChanged(this, &SNamingTokenDataTreeViewWidget::OnTreeViewItemSelectionChanged)
			.OnMouseButtonDoubleClick(this, &SNamingTokenDataTreeViewWidget::OnTreeViewItemDoubleClicked)
		]
	];

	FilterTreeItems(FString());
}

void SNamingTokenDataTreeViewWidget::PopulateTreeItems(const FNamingTokenFilterArgs& InFilterArgs)
{
	RootTreeItems.Reset();
	if (GEngine)
	{
		if (const UNamingTokensEngineSubsystem* NamingTokensSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>())
		{
			TArray<FString> Namespaces = NamingTokensSubsystem->GetAllNamespaces(/*bIncludePrivateNamespaces*/ true);
			RootTreeItems.Reserve(Namespaces.Num());

			// Sort namespaces so any matching the priority list are displayed first, in that order.
			// Any other namespaces preserve their order. It's better to do this before building out final lists
			// so we can record the proper index order for navigation next.
			if (!NamespacePriority.IsEmpty())
			{
				TMap<FString, int32> NamespaceOrder;
				NamespaceOrder.Reserve(NamespacePriority.Num());
				for (int32 Idx = 0; Idx < NamespacePriority.Num(); ++Idx)
				{
					NamespaceOrder.Add(NamespacePriority[Idx], Idx);
				}

				Namespaces.StableSort([&](const FString& A, const FString& B)
				{
					const int32* IndexA = NamespaceOrder.Find(A);
					const int32* IndexB = NamespaceOrder.Find(B);

					if (IndexA && IndexB)
					{
						return *IndexA < *IndexB;
					}

					if (IndexA)
					{
						return true;
					}

					// Keep the original order.
					return false;
				});
			}
			
			int32 TotalCount = 0;
			
			for (const FString& Namespace : Namespaces)
			{
				FNamingTokenDataTreeItemPtr NamespaceTreeItem = MakeShared<FNamingTokenDataTreeItem>();
				NamespaceTreeItem->Namespace = Namespace;
				NamespaceTreeItem->ItemIndex = TotalCount++;
				NamespaceTreeItem->bIsGlobal = NamingTokensSubsystem->IsGlobalNamespaceRegistered(Namespace);
				
				if (const UNamingTokens* NamingTokens = NamingTokensSubsystem->GetNamingTokens(Namespace))
				{
					const bool bIsPrivate = NamingTokens->IsPrivateNamespace();
					// Filter out any private namespaces unless specified.
					if (bIsPrivate && !InFilterArgs.AdditionalNamespacesToInclude.Contains(Namespace))
					{
						continue;
					}

					NamespaceTreeItem->NamespaceDisplayName = NamingTokens->GetNamespaceDisplayName();
					
					const TArray<FNamingTokenData> Tokens = NamingTokens->GetAllTokens();
					if (Tokens.Num() == 0)
					{
						// Don't display empty namespaces.
						continue;
					}
					
					NamespaceTreeItem->ChildItems.Reserve(Tokens.Num());
					
					for (const FNamingTokenData& Token : Tokens)
					{
						FNamingTokenDataTreeItemPtr TokenTreeItem = MakeShared<FNamingTokenDataTreeItem>();
						TokenTreeItem->NamingTokenData = MakeShared<FNamingTokenData>(Token);
						TokenTreeItem->Namespace = Namespace;
						TokenTreeItem->bIsGlobal = NamespaceTreeItem->bIsGlobal;
						TokenTreeItem->ItemIndex = TotalCount++;
						NamespaceTreeItem->ChildItems.Add(TokenTreeItem);
					}
				}

				if (NamespaceTreeItem->ChildItems.Num() > 0)
				{
					RootTreeItems.Add(NamespaceTreeItem);
				}
			}
		}
	}
}

void SNamingTokenDataTreeViewWidget::FilterTreeItems(const FString& InFilterText)
{
	bool bSelectingNamespace = false;
	FString SanitizedFilterText = InFilterText;
	SanitizedFilterText.RemoveSpacesInline();
	if (LastFilterText == SanitizedFilterText && FilteredRootTreeItems.Num() > 0)
	{
		if (const FNamingTokenDataTreeItemPtr CurrentlySelectedItem = GetSelectedItem())
		{
			// This is used at the end to skip selecting the namespace again if that's all we're showing.
			// We may need to move this logic elsewhere in case filtering is called when selecting for some reason.
			bSelectingNamespace = SanitizedFilterText == CurrentlySelectedItem->Namespace && CurrentlySelectedItem->IsNamespace();
		}
	}
	LastFilterText = SanitizedFilterText;
	FString CompleteOrPartialNamespace = UE::NamingTokens::Utils::GetNamespaceFromTokenKey(SanitizedFilterText);
	FString PartialToken = UE::NamingTokens::Utils::RemoveNamespaceFromTokenKey(SanitizedFilterText);

	TokenFilterText = PartialToken;
	NamespaceFilterText = CompleteOrPartialNamespace;
	
	const bool bSearchNamespaceAndToken = CompleteOrPartialNamespace.IsEmpty();
	if (bSearchNamespaceAndToken)
	{
		// Can't discern either part, assume searching for namespaces.
		
		CompleteOrPartialNamespace = SanitizedFilterText;
		PartialToken = SanitizedFilterText;
	}

	// We have a complete namespace we require.
	const bool bExactNamespaceMatch = !bSearchNamespaceAndToken;
	
	FilteredRootTreeItems.Reset();

	// The best matched namespace or token index found while filtering.
	TTuple<int32, float> BestMatchedIndex = { 0, 0.f };
	
	int32 TotalCount = 0;
	
	for (const FNamingTokenDataTreeItemPtr& TreeItem : RootTreeItems)
	{
		// Copy our item so we can modify the contained children.
		const FNamingTokenDataTreeItemPtr FilteredNamespace = MakeShared<FNamingTokenDataTreeItem>(*TreeItem);
		FilteredNamespace->ChildItems.Reset();
		
		// Check if the namespace matches.
		const float NamespaceMatchPercent = TreeItem->MatchesFilter(CompleteOrPartialNamespace, bExactNamespaceMatch);
		if (bSearchNamespaceAndToken || NamespaceMatchPercent > 0.f)
		{
			int32 IterationCount = TotalCount;
			FilteredNamespace->ItemIndex = IterationCount++;

			TTuple<int32, float> HighestTokenIndexScore = TTuple<int32, float>(0, 0.f);
			for (const FNamingTokenDataTreeItemPtr& ChildItem : TreeItem->ChildItems)
			{
				// The token itself.
				const FNamingTokenDataTreeItemPtr FilteredChild = MakeShared<FNamingTokenDataTreeItem>(*ChildItem);
				const float TokenMatchPercent = FilteredChild->MatchesFilter(PartialToken);
				if (TokenMatchPercent > 0.f || NamespaceMatchPercent > 0.f)
				{
					FilteredChild->ItemIndex = IterationCount++;
					FilteredNamespace->ChildItems.Add(FilteredChild);

					if (TokenMatchPercent > HighestTokenIndexScore.Get<1>())
					{
						HighestTokenIndexScore = TTuple<int32, float>(FilteredChild->ItemIndex, TokenMatchPercent);
					}
				}
			}

			// Don't include namespace if it or its tokens don't match.
			if (NamespaceMatchPercent || FilteredNamespace->ChildItems.Num() > 0)
			{
				// Only set persistent count when we know were committing this result.
				TotalCount = IterationCount;
				FilteredRootTreeItems.Add(FilteredNamespace);

				// Determine the best match between namespace and token results.
				TTuple<int32, float> HighestOverallScore = { 0, 0.f };

				bool bUsingNamespaceScore = false;
				// Prioritize namespace first unless we're already at an exact match, because then we're searching for tokens
				// inside that namespace.
				if (!bExactNamespaceMatch && NamespaceMatchPercent >= HighestTokenIndexScore.Get<1>() && NamespaceMatchPercent > 0.f)
				{
					HighestOverallScore = TTuple<int32, float>(FilteredNamespace->ItemIndex, NamespaceMatchPercent);
					bUsingNamespaceScore = true;
				}
				else
				{
					HighestOverallScore = HighestTokenIndexScore;
				}

				// Record the best match overall so far.
				if (HighestOverallScore.Get<1>() > BestMatchedIndex.Get<1>()
					// Namespaces should take priority.
					|| (bUsingNamespaceScore && HighestOverallScore.Get<1>() >= 0.75f && HighestOverallScore.Get<1>() >= BestMatchedIndex.Get<1>()))
				{
					BestMatchedIndex = MoveTemp(HighestOverallScore);
				}
			}
			
			if (bExactNamespaceMatch)
			{
				// We've already found an exact match to our namespace, no need to continue searching namespaces.
				break;
			}
		}
	}
	
	TreeView->RequestListRefresh();
	TreeView->SetExpansionStateFromItems(FilteredRootTreeItems);

	if (FilteredRootTreeItems.Num() > 0)
	{
		SetItemFromIndex(bSelectingNamespace ? 1 : BestMatchedIndex.Get<0>());
	}
}

bool SNamingTokenDataTreeViewWidget::ForwardKeyEventForNavigation(const FKeyEvent& InKeyEvent)
{
	check(TreeView.IsValid());

	const FKey Key = InKeyEvent.GetKey();
	if (Key == EKeys::Tab || Key == EKeys::Enter)
	{
		FinalizeSelection(GetSelectedItem());
		return true;
	}
	
	const EUINavigation Direction = FSlateApplicationBase::Get().GetNavigationDirectionFromKey(InKeyEvent);
	switch (Direction)
	{
		case EUINavigation::Down:
			SetItemFromIndex(CurrentSelectionIndex + 1);
			return true;
		case EUINavigation::Up:
			if (CurrentSelectionIndex > 0)
			{
				SetItemFromIndex(CurrentSelectionIndex - 1);
			}
			return true;
		default:
			break;
	}
	return false;
}

FNamingTokenDataTreeItemPtr SNamingTokenDataTreeViewWidget::GetSelectedItem() const
{
	return GetItemFromIndex(CurrentSelectionIndex, FilteredRootTreeItems);
}

FNamingTokenDataTreeItemPtr SNamingTokenDataTreeViewWidget::GetItemFromIndex(int32 InIndex) const
{
	return GetItemFromIndex(InIndex, FilteredRootTreeItems);
}

void SNamingTokenDataTreeViewWidget::NotifyFocusReceived()
{
	OnFocusedDelegate.ExecuteIfBound();
}

FReply SNamingTokenDataTreeViewWidget::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	NotifyFocusReceived();
	return SCompoundWidget::OnFocusReceived(MyGeometry, InFocusEvent);
}

FNamingTokenDataTreeItemPtr SNamingTokenDataTreeViewWidget::GetItemFromIndex(int32 InIndex, const TArray<FNamingTokenDataTreeItemPtr>& InItems) const
{
	for (int32 Idx = 0; Idx < InItems.Num(); ++Idx)
	{
		const FNamingTokenDataTreeItemPtr& TreeItem = InItems[Idx];
		if (TreeItem->ItemIndex == InIndex)
		{
			return TreeItem;
		}

		if (TreeItem->ChildItems.Num() > 0)
		{
			const FNamingTokenDataTreeItemPtr& ChildItem = GetItemFromIndex(InIndex, TreeItem->ChildItems);
			if (ChildItem.IsValid())
			{
				return ChildItem;
			}
		}
	}

	return nullptr;
}

void SNamingTokenDataTreeViewWidget::SetItemFromIndex(int32 InIndex)
{
	if (const FNamingTokenDataTreeItemPtr Item = GetItemFromIndex(InIndex))
	{
		ClearCurrentSelection();
		
		CurrentSelectionIndex = InIndex;
		TreeView->SetItemSelection(Item, true);
		if (!TreeView->IsItemVisible(Item))
		{
			TreeView->RequestScrollIntoView(Item);
		}
	}
}

void SNamingTokenDataTreeViewWidget::ClearCurrentSelection()
{
	if (const FNamingTokenDataTreeItemPtr CurrentSelection = GetSelectedItem())
	{
		TreeView->SetItemSelection(CurrentSelection, false);
		CurrentSelectionIndex = INDEX_NONE;
	}
}

void SNamingTokenDataTreeViewWidget::FinalizeSelection(FNamingTokenDataTreeItemPtr SelectedItem)
{
	ItemSelectedDelegate.ExecuteIfBound(SelectedItem);
}

TSharedRef<ITableRow> SNamingTokenDataTreeViewWidget::OnGenerateRowForTree(FNamingTokenDataTreeItemPtr Item,
                                                                           const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SNamingTokenDataTreeViewRow, OwnerTable).Item(Item);
}

void SNamingTokenDataTreeViewWidget::OnGetChildrenForTree(FNamingTokenDataTreeItemPtr InParent, TArray<FNamingTokenDataTreeItemPtr>& OutChildren)
{
	OutChildren.Append(InParent->ChildItems);
}

void SNamingTokenDataTreeViewWidget::OnItemExpansionChanged(FNamingTokenDataTreeItemPtr TreeItem, bool bIsExpanded) const
{
	
}

void SNamingTokenDataTreeViewWidget::OnTreeViewItemSelectionChanged(FNamingTokenDataTreeItemPtr SelectedItem, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo == ESelectInfo::OnMouseClick && SelectedItem.IsValid())
	{
		ClearCurrentSelection();
		CurrentSelectionIndex = SelectedItem->ItemIndex;
		FinalizeSelection(SelectedItem);
	}
	ItemSelectionChangedDelegate.ExecuteIfBound(SelectedItem, SelectInfo);
}

void SNamingTokenDataTreeViewWidget::OnTreeViewItemDoubleClicked(FNamingTokenDataTreeItemPtr SelectedItem)
{
	FinalizeSelection(SelectedItem);
}

#undef LOCTEXT_NAMESPACE
