// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NamingTokenData.h"
#include "NamingTokensEngineSubsystem.h"
#include "Utils/NamingTokenUtils.h"

#include "Widgets/Views/STreeView.h"

/**
 * Represents naming token items in our tree view.
 */
struct FNamingTokenDataTreeItem : TSharedFromThis<FNamingTokenDataTreeItem>
{
	/** The token data, null on namespace entries. */
	TSharedPtr<FNamingTokenData> NamingTokenData;

	/** The namespace for this token. May be set regardless of entry type. */
	FString Namespace;
	/** The friendly display name for the namespace. */
	FText NamespaceDisplayName;

	/** Children of a namespace. */
	TArray<TSharedPtr<FNamingTokenDataTreeItem>> ChildItems;
	/** The parent owning this entry. */
	TWeakPtr<FNamingTokenDataTreeItem> Parent;

	/** The flattened item index in the tree. */
	int32 ItemIndex = INDEX_NONE;

	/** If this entry is globally registered in the naming tokens subsystem. */
	bool bIsGlobal = false;
	
	bool ShouldItemBeExpanded() const { return true; }

	/** Checks if this item matches a filter returning a simple percentage score on the match. */
	float MatchesFilter(const FString& FilterText, bool bIsExactNamespaceMatch = false) const
	{
		if (FilterText.IsEmpty())
		{
			return 0.5f;
		}

		if (IsNamespace())
		{
			if (Namespace == FilterText) // Exact match requires true namespace being typed out...
			{
				return 1.f;
			}
			
			if (!bIsExactNamespaceMatch &&
				(Namespace.StartsWith(FilterText, ESearchCase::IgnoreCase) // ...Otherwise, we can check partial match and the display name.
				|| NamespaceDisplayName.ToString().StartsWith(FilterText, ESearchCase::IgnoreCase)))
			{
				return 0.75f;
			}

			return 0.f;
		}

		check(NamingTokenData.IsValid());

		if (NamingTokenData->TokenKey.Equals(FilterText, ESearchCase::CaseSensitive))
		{
			return 1.f;
		}
		
		if (NamingTokenData->TokenKey.Equals(FilterText, ESearchCase::IgnoreCase))
		{
			return 0.9f;
		}
		
		if (NamingTokenData->TokenKey.StartsWith(FilterText, ESearchCase::IgnoreCase)
			|| NamingTokenData->DisplayName.ToString().StartsWith(FilterText, ESearchCase::IgnoreCase))
		{
			return 0.75f;
		}

		return 0.f;
	}

	/** True iff this is considered a namespace entry. */
	bool IsNamespace() const
	{
		return !NamingTokenData.IsValid();
	}
	
	/** Convert the item to the best text representation. */
	FString ToString(bool bFullyQualified) const
	{
		if (NamingTokenData.IsValid())
		{
			return bFullyQualified ?
				UE::NamingTokens::Utils::CombineNamespaceAndTokenKey(Namespace, NamingTokenData->TokenKey)
					: NamingTokenData->TokenKey;
		}
		return Namespace;
	}
	
	bool operator==(const FNamingTokenDataTreeItem& Other) const
	{
		return NamingTokenData == Other.NamingTokenData && Namespace == Other.Namespace;
	}
};

typedef TSharedPtr<FNamingTokenDataTreeItem> FNamingTokenDataTreeItemPtr;

/**
 * A tree view for naming token filtering.
 */
class SNamingTokenDataTreeView : public STreeView<FNamingTokenDataTreeItemPtr>
{
public:
	void Construct(const FArguments& InArgs, const TSharedPtr<class SNamingTokenDataTreeViewWidget>& InTreeViewWidget);

	/** Recursively set expansion state of tree view to match items. */
	void SetExpansionStateFromItems(const TArray<FNamingTokenDataTreeItemPtr>& InTreeItems);

	/** Pointer to our owning widget. */
	const TWeakPtr<SNamingTokenDataTreeViewWidget>& GetOwningTreeViewWidget() const { return OwningTreeViewWidget; }

	// ~Begin STreeView
	virtual bool HasKeyboardFocus() const override { return false; }
	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	// ~End STreeView
private:
	/** The owning tree view widget. */
	TWeakPtr<SNamingTokenDataTreeViewWidget> OwningTreeViewWidget;
};

/**
 * Each row of the tree view.
 */
class SNamingTokenDataTreeViewRow final : public SMultiColumnTableRow<FNamingTokenDataTreeItemPtr>
{
public:
	SLATE_BEGIN_ARGS(SNamingTokenDataTreeViewRow) {}

	/** The list item for this row. */
	SLATE_ARGUMENT(FNamingTokenDataTreeItemPtr, Item)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable);

	// ~Begin SMultiColumnTableRow interface
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	// ~End SMultiColumnTableRow interface

private:
	static const FEditableTextBoxStyle& GetCustomEditableTextBoxStyle();

private:
	/** The item associated with this row of data. */
	TWeakPtr<FNamingTokenDataTreeItem> Item;
};

/**
 * Widget for displaying namespaces and tokens to choose from.
 */
class SNamingTokenDataTreeViewWidget final : public SCompoundWidget
{
	using FOnSelectionChanged = STreeView<FNamingTokenDataTreeItemPtr>::FOnSelectionChanged;

	DECLARE_DELEGATE_OneParam(FOnItemSelected, FNamingTokenDataTreeItemPtr);
	
public:
	SLATE_BEGIN_ARGS(SNamingTokenDataTreeViewWidget) {}
		/** If a selection is changed, such as arrow keys or a single click. */
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		/** If an item is formally selected, such as by pressing enter or a double click. */
		SLATE_EVENT(FOnItemSelected, OnItemSelected)
		/** Event if this widget or its children receives focus. */
		SLATE_EVENT(FSimpleDelegate, OnFocused)
		/** Additional filter args to apply. */
		SLATE_ARGUMENT(FNamingTokenFilterArgs, FilterArgs)
		/** Display priority of namespaces. */
		SLATE_ARGUMENT(TArray<FString>, NamespacePriority)
	SLATE_END_ARGS()

	virtual ~SNamingTokenDataTreeViewWidget() override;
	
	void Construct(const FArguments& InArgs);

	/** Populate all possible tree items */
	void PopulateTreeItems(const FNamingTokenFilterArgs& InFilterArgs);

	/** Filter down to specific tokens and namespaces. */
	void FilterTreeItems(const FString& InFilterText);
	
	/**
	 * Provide a key event from the user to determine a navigation path, highlighting an item.
	 * @return True if the key event contained a recognized navigation key.
	 */
	bool ForwardKeyEventForNavigation(const FKeyEvent& InKeyEvent);

	/** Retrieve the currently selected item. */
	FNamingTokenDataTreeItemPtr GetSelectedItem() const;

	/** Retrieve an item given its absolute index. */
	FNamingTokenDataTreeItemPtr GetItemFromIndex(int32 InIndex) const;

	/** Get the token filter text applied. */
	const FString& GetTokenFilterText() const { return TokenFilterText; }

	/** Get the namespace filter text applied. */
	const FString& GetNamespaceFilterText() const { return NamespaceFilterText; }

	/** Informs us we're receiving focus, such as from a child of the treeview. */
	void NotifyFocusReceived();

	// ~Begin SCompoundWidget
	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	// ~End SCompoundWidget
	
private:
	/** Find an item given its index. */
	FNamingTokenDataTreeItemPtr GetItemFromIndex(int32 InIndex, const TArray<FNamingTokenDataTreeItemPtr>& InItems) const;
	
	/** Set the currently selected item given an absolute index. */
	void SetItemFromIndex(int32 InIndex);

	/** Clears the currently selected item. */
	void ClearCurrentSelection();
	
	/** Officially complete and report the selection. Called from tab or double-click. */
	void FinalizeSelection(FNamingTokenDataTreeItemPtr SelectedItem);
	
public:
	static FName PrimaryColumnName() { return "Primary"; }
	
private:
	/** Called by STreeView for each row being generated. */
	TSharedRef<ITableRow> OnGenerateRowForTree(FNamingTokenDataTreeItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable);
	
	/** Called by STreeView to get child items for the specified parent item. */
	void OnGetChildrenForTree(FNamingTokenDataTreeItemPtr InParent, TArray<FNamingTokenDataTreeItemPtr>& OutChildren);
	
	/** Called when an item in the tree has been collapsed or expanded. */
	void OnItemExpansionChanged(FNamingTokenDataTreeItemPtr TreeItem, bool bIsExpanded) const;

	/** Called when the tree view changes selection. */
	void OnTreeViewItemSelectionChanged(FNamingTokenDataTreeItemPtr SelectedItem, ESelectInfo::Type SelectInfo);

	/** Called when an item in the tree view is double-clicked. */
	void OnTreeViewItemDoubleClicked(FNamingTokenDataTreeItemPtr SelectedItem);
	
private:
	/** Our tree view widget. */
	TSharedPtr<SNamingTokenDataTreeView> TreeView;

	/** The top most items on the tree view without a filter. */
	TArray<FNamingTokenDataTreeItemPtr> RootTreeItems;

	/** Items filtered in a search. */
	TArray<FNamingTokenDataTreeItemPtr> FilteredRootTreeItems;

	/** Order to display namespaces. */
	TArray<FString> NamespacePriority;

	/** The last filter applied. */
	FString LastFilterText;

	/** Last token filter text applied. */
	FString TokenFilterText;

	/** Last namespace filter text applied. */
	FString NamespaceFilterText;
	
	/** Index of tree item currently selected. */
	int32 CurrentSelectionIndex = 0;

	/** Delegate for when selection changes. */
	FOnSelectionChanged ItemSelectionChangedDelegate;

	/** Delegate for when an item is fully selected. */
	FOnItemSelected ItemSelectedDelegate;

	/** Delegate if we have received focus. */
	FSimpleDelegate OnFocusedDelegate;
};