// Copyright Epic Games, Inc. All Rights Reserved.

#include "SNamingTokensEditableTextBox.h"

#include "NamingTokensStringSyntaxHighlighterMarshaller.h"
#include "SNamingTokensDataTreeView.h"
#include "Utils/NamingTokenUtils.h"

#include "Engine/Engine.h"
#include "Framework/Application/SlateApplication.h"
#include "Internationalization/Regex.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SPopUpErrorText.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SNamingTokensEditableTextBox"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SNamingTokensEditableTextBox::Construct(const FArguments& InArgs)
{
	if (InArgs._Style == nullptr)
	{
		SetNormalStyle(&FAppStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"));		
	}
	else
	{
		SetNormalStyle(InArgs._Style);
	}

	if (InArgs._ArgumentStyle != nullptr)
	{
		SetArgumentStyle(InArgs._ArgumentStyle);
	}
	
	Marshaller = FNamingTokensStringSyntaxHighlighterMarshaller::Create(
		FNamingTokensStringSyntaxHighlighterMarshaller::FSyntaxTextStyle(TextBoxStyle->TextStyle,
			InArgs._ArgumentStyle ? *InArgs._ArgumentStyle
			: FNamingTokensStringSyntaxHighlighterMarshaller::GetDefaultArgumentStyle()));
	
	static const FVector2D ScrollBarThickness(9.0f, 9.0f);
	static const FMargin ErrorWidgetPadding(3, 0); 

	TokenizedText = InArgs._Text;
	ResolvedText = InArgs._ResolvedText;
	ValidArguments = InArgs._ValidArguments;

	OnValidateTokenizedText = InArgs._OnValidateTokenizedText;
	OnValidateResolvedText = InArgs._OnValidateResolvedText;
	OnTokenizedTextChanged = InArgs._OnTextChanged;
	OnTokenizedTextCommitted = InArgs._OnTextCommitted;
	OnPreEvaluateTokens = InArgs._OnPreEvaluateNamingTokens;
	OnTokenizedTextEvaluated = InArgs._OnTokenizedTextEvaluated;

	ShouldEvaluateTokens = InArgs._ShouldEvaluateTokens;
	EvaluationFrequency = InArgs._EvaluationFrequency;
	CanDisplayResolvedText = InArgs._CanDisplayResolvedText;

	DisplayTokenIcon = InArgs._DisplayTokenIcon;
	DisplayErrorMessage = InArgs._DisplayErrorMessage;
	DisplayBorderImage = InArgs._DisplayBorderImage;
	TextBoxPaddingOverride = InArgs._TextBoxPadding;

	bSupportInlineEdit = InArgs._SupportInlineEdit;
	bEnableSuggestionDropdown = InArgs._EnableSuggestionDropdown;
	bFullyQualifyGlobalTokenSuggestions = InArgs._FullyQualifyGlobalTokenSuggestions;
	bFullyQualifyFilteredNamespaces = InArgs._FullyQualifyFilteredNamespaces;
	bShowUnknownTokenWarning = InArgs._ShowUnknownTokenWarning;
	bShowUnsetTokenWarning = InArgs._ShowUnsetTokenWarning;
	SetNamespaceSuggestionPriority(InArgs._NamespaceSuggestionPriority);
	
	SetFilterArgs(InArgs._FilterArgs);
	SetContexts(InArgs._Contexts);
	
	ErrorReporting = SNew(SPopupErrorText);
	ErrorReporting->SetError(FText::GetEmpty());

	// Create a Slate timer that will re-evaluate the token text at the specified evaluation frequency
	if (EvaluationFrequency.Get() > 0.0f)
	{
		RegisterActiveTimer(EvaluationFrequency.Get(), FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime) -> EActiveTimerReturnType
			{
				EvaluateNamingTokens();
				return EActiveTimerReturnType::Continue;
			}));
	}

	BoundText = MakeAttributeSP(this, &SNamingTokensEditableTextBox::GetText);
	
	const FMargin TextBoxPadding = TextBoxPaddingOverride.IsSet() ? TextBoxPaddingOverride.Get() : TextBoxStyle->Padding;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(this, &SNamingTokensEditableTextBox::GetBorderImage)
		.BorderBackgroundColor(this, &SNamingTokensEditableTextBox::GetBorderBackgroundColor)
		.ForegroundColor(this, &SMultiLineEditableTextBox::GetForegroundColor)
		.Padding(0)
		[
			SAssignNew(Box, SHorizontalBox)

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(4, 0, 0, 0)
			[
				SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.TokenTextBox"))
					.Visibility_Lambda([this]()
					{
						return DisplayTokenIcon.Get(false) ? EVisibility::Visible : EVisibility::Collapsed;
					})
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillWidth(1.0f)
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.Padding(TextBoxPadding)
				.FillHeight(1.0f)
				[
					SAssignNew(DropdownAnchor, SMenuAnchor)
					.Method(EPopupMethod::UseCurrentWindow)
					.Placement(MenuPlacement_ComboBox)
					// The drop down filter menu.
					.OnGetMenuContent(this, &SNamingTokensEditableTextBox::OnGetDropdownContent)
					// False to prevent the menu from being dismissed by the application. Otherwise, it gets dismissed
					// when the user clicks on the dropdown, because we send a keyboard focus event back to the text widget.
					.UseApplicationMenuStack(false)
					.Visibility(EVisibility::Visible)
					.Content()
					[
						SAssignNew(EditableText, SMultiLineEditableText)
						.Text(BoundText)
						.TextStyle(&TextBoxStyle->TextStyle)
						.Marshaller(Marshaller)
						.AllowMultiLine(InArgs._AllowMultiLine)
						.IsReadOnly(InArgs._IsReadOnly)
						.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
						.Margin(0.0f)
						.OnTextChanged(this, &SNamingTokensEditableTextBox::OnEditableTextChanged)
						.OnTextCommitted(this, &SNamingTokensEditableTextBox::OnEditableTextCommitted)
						.OnKeyDownHandler(this, &SNamingTokensEditableTextBox::OnEditableTextKeyDown)
						.OnKeyCharHandler(this, &SNamingTokensEditableTextBox::OnEditableTextKeyChar)
						.OnCursorMoved(this, &SNamingTokensEditableTextBox::OnEditableTextCursorMoved)
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(HScrollBarBox, SBox)
					.Padding(TextBoxStyle->HScrollBarPadding)
					[
						SAssignNew(HScrollBar, SScrollBar)
						.Style(&TextBoxStyle->ScrollBarStyle)
						.Orientation(Orient_Horizontal)
						.Thickness(ScrollBarThickness)
					]
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(VScrollBarBox, SBox)
				.Padding(TextBoxStyle->VScrollBarPadding)
				[
					SAssignNew(VScrollBar, SScrollBar)
					.Style(&TextBoxStyle->ScrollBarStyle)
					.Orientation(Orient_Vertical)
					.Thickness(ScrollBarThickness)
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(4, 0, 0, 0))
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Warning"))
				.ColorAndOpacity(FStyleColors::Warning)
				.Visibility_Lambda([this]()
				{
					return (!UnknownTokenKeys.IsEmpty() || !UnsetTokenKeys.IsEmpty()) ? EVisibility::Visible : EVisibility::Collapsed;
				})
				.ToolTipText(this, &SNamingTokensEditableTextBox::GetUnresolvedTokenWarningTooltip)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(ErrorWidgetPadding)
			[
				SNew(SBox)
				.Visibility_Lambda([this]()
				{
					return DisplayErrorMessage.Get(false) ? EVisibility::Visible : EVisibility::Collapsed;
				})
				[
					ErrorReporting->AsWidget()
				]
			]
		]
	];
	
	if (bSupportInlineEdit)
	{
		SetVisibility(EVisibility::HitTestInvisible);
	}

	// Evaluate the initial text, validate it, and evaluate tokens
	const FText InitialText = TokenizedText.Get();
	ValidateTokenizedText(InitialText);
	EvaluateNamingTokens();
}

void SNamingTokensEditableTextBox::SetTokenizedText(const FText& InText)
{
	TokenizedText.Set(InText);
	EvaluateNamingTokens();
}

const FText& SNamingTokensEditableTextBox::GetResolvedText() const
{
	return ResolvedText.Get();
}

const FText& SNamingTokensEditableTextBox::GetTokenizedText() const
{
	return TokenizedText.Get();
}

void SNamingTokensEditableTextBox::SetNormalStyle(const FEditableTextBoxStyle* InStyle)
{
	if (InStyle)
	{
		TextBoxStyle = InStyle;
	}
	else
	{
		FArguments Defaults;
		TextBoxStyle = Defaults._Style;
	}

	check(TextBoxStyle);
	
	if (EditableText.IsValid())
	{
		EditableText->SetTextStyle(&TextBoxStyle->TextStyle);
	}

	if (Marshaller.IsValid())
	{
		Marshaller->SetNormalStyle(TextBoxStyle->TextStyle);
		Marshaller->MakeDirty();
	}
}

void SNamingTokensEditableTextBox::SetArgumentStyle(const FTextBlockStyle* InArgumentStyle)
{
	ArgumentStyle = InArgumentStyle;
	
	if (Marshaller.IsValid() && InArgumentStyle)
	{
		Marshaller->SetArgumentStyle(*InArgumentStyle);	
		Marshaller->MakeDirty();
	}
}

void SNamingTokensEditableTextBox::SetSupportInlineEdit(bool bInSupportInlineEdit)
{
	bSupportInlineEdit = bInSupportInlineEdit;
}

void SNamingTokensEditableTextBox::SetEnableSuggestionDropdown(bool bInEnableSuggestionDropdown)
{
	bEnableSuggestionDropdown = bInEnableSuggestionDropdown;
}

void SNamingTokensEditableTextBox::SetFilterArgs(const FNamingTokenFilterArgs& InFilterArgs)
{
	NamingTokenFilterArgs = InFilterArgs;
}

void SNamingTokensEditableTextBox::SetFullyQualifyFilteredNamespaces(bool bNewValue)
{
	bFullyQualifyFilteredNamespaces = bNewValue;
}

void SNamingTokensEditableTextBox::SetNamespaceSuggestionPriority(const TArray<FString>& InNamespaces)
{
	NamespaceSuggestionPriority = InNamespaces;
}

void SNamingTokensEditableTextBox::SetContexts(const TArray<UObject*>& InContexts)
{
	NamingTokenContexts = InContexts;
}

void SNamingTokensEditableTextBox::SetShowUnknownTokenWarning(bool bNewValue)
{
	if (bShowUnknownTokenWarning != bNewValue)
	{
		bShowUnknownTokenWarning = bNewValue;
		// If the toggle was just disabled, clear stale entries so the warning icon hides immediately.
		if (!bShowUnknownTokenWarning)
		{
			UnknownTokenKeys.Reset();
		}
		EvaluateNamingTokens();
	}
}

void SNamingTokensEditableTextBox::SetShowUnsetTokenWarning(bool bNewValue)
{
	if (bShowUnsetTokenWarning != bNewValue)
	{
		bShowUnsetTokenWarning = bNewValue;
		if (!bShowUnsetTokenWarning)
		{
			UnsetTokenKeys.Reset();
		}
		EvaluateNamingTokens();
	}
}

void SNamingTokensEditableTextBox::EvaluateNamingTokens()
{
	if (ShouldEvaluateTokens.Get() && !ResolvedText.IsBound())
	{
		OnPreEvaluateTokens.ExecuteIfBound();

		UNamingTokensEngineSubsystem* TokenSubsystem = GEngine->GetEngineSubsystem<UNamingTokensEngineSubsystem>();
		const FNamingTokenResultData EvaluationData = TokenSubsystem->EvaluateTokenText(TokenizedText.Get(),
			NamingTokenFilterArgs, NamingTokenContexts);

		ResolvedText.Set(EvaluationData.EvaluatedText);

		// Refresh the list of unknown and unset token keys. Each array is only populated when its corresponding warning
		// is enabled so the icon-visibility lambda and tooltip naturally honor each toggle without an extra gate.
		if (bShowUnknownTokenWarning || bShowUnsetTokenWarning)
		{
			UnknownTokenKeys.Reset();
			UnsetTokenKeys.Reset();

			for (const FNamingTokenValueData& TokenValue : EvaluationData.TokenValues)
			{
				if (!TokenValue.bWasEvaluated)
				{
					if (bShowUnknownTokenWarning)
					{
						UnknownTokenKeys.AddUnique(TokenValue.TokenKey);
					}
				}
				else if (TokenValue.TokenValue.IsEmpty())
				{
					if (bShowUnsetTokenWarning)
					{
						UnsetTokenKeys.AddUnique(TokenValue.TokenKey);
					}
				}
			}
		}

		if (OnValidateResolvedText.IsBound())
		{
			FText ErrorMessage;
			const bool bIsValid = OnValidateResolvedText.Execute(ResolvedText.Get(), ErrorMessage);

			if (!HasKeyboardFocus())
			{
				SetError(bIsValid ? FText::GetEmpty() : ErrorMessage);
			}
		}

		OnTokenizedTextEvaluated.ExecuteIfBound(ResolvedText.Get());
	}
}

FText SNamingTokensEditableTextBox::GetUnresolvedTokenWarningTooltip() const
{
	if (UnknownTokenKeys.IsEmpty() && UnsetTokenKeys.IsEmpty())
	{
		return FText::GetEmpty();
	}

	// Build one section per failure mode so the user sees exactly which tokens need attention and the specific reason each is unresolved.
	TArray<FText> Sections;

	auto WrapAndJoinKeys = [](const TArray<FString>& Keys)
	{
		TArray<FString> Wrapped;
		Wrapped.Reserve(Keys.Num());
		for (const FString& Key : Keys)
		{
			Wrapped.Add(FString::Printf(TEXT("{%s}"), *Key));
		}
		return FString::Join(Wrapped, TEXT(", "));
	};

	if (!UnknownTokenKeys.IsEmpty())
	{
		Sections.Add(FText::Format(
			LOCTEXT("UnknownTokenKeysWarning", "The following tokens are not recognized. Check the spelling and/or namespace, or consider removing:\n    {0}"),
			FText::FromString(WrapAndJoinKeys(UnknownTokenKeys))));
	}

	if (!UnsetTokenKeys.IsEmpty())
	{
		Sections.Add(FText::Format(
			LOCTEXT("UnsetTokenKeysWarning", "The following tokens are recognized but currently evaluate to an empty value. Set the value, or consider removing:\n    {0}"),
			FText::FromString(WrapAndJoinKeys(UnsetTokenKeys))));
	}

	return FText::Join(FText::FromString(TEXT("\n\n")), Sections);
}

FText SNamingTokensEditableTextBox::GetText() const
{
	if (HasKeyboardFocus())
	{
		return TokenizedText.Get();
	}
	else
	{
		if (ResolvedText.IsSet() && CanDisplayResolvedText.Get(true))
		{
			return ResolvedText.Get();
		}
		else
		{
			return TokenizedText.Get();
		}
	}
}

const FSlateBrush* SNamingTokensEditableTextBox::GetBorderImage() const
{
	check(TextBoxStyle);
	check(EditableText.IsValid());

	if (DisplayBorderImage.IsSet() && !DisplayBorderImage.Get())
	{
		return nullptr;
	}

	if (EditableText->IsTextReadOnly())
	{
		return &TextBoxStyle->BackgroundImageReadOnly;
	}
	else if (EditableText->HasKeyboardFocus())
	{
		return &TextBoxStyle->BackgroundImageFocused;
	}
	else if (EditableText->IsHovered())
	{
		return &TextBoxStyle->BackgroundImageHovered;
	}
	else
	{
		return &TextBoxStyle->BackgroundImageNormal;
	}
}

FSlateColor SNamingTokensEditableTextBox::GetBorderBackgroundColor() const
{
	check(TextBoxStyle);
	return TextBoxStyle->BackgroundColor;
}

FSlateColor SNamingTokensEditableTextBox::GetForegroundColor() const
{
	check(TextBoxStyle);

	return HasKeyboardFocus()
		? TextBoxStyle->FocusedForegroundColor
		: TextBoxStyle->ForegroundColor;
}

void SNamingTokensEditableTextBox::SetError(const FText& InError) const
{
	ErrorReporting->SetError(InError);
}

bool SNamingTokensEditableTextBox::SupportsKeyboardFocus() const
{
	return true;
}

bool SNamingTokensEditableTextBox::HasKeyboardFocus() const
{
	// Since keyboard focus is forwarded to our editable text, we will test it instead
	return SCompoundWidget::HasKeyboardFocus() || EditableText->HasKeyboardFocus() || (DropdownAnchor.IsValid() && DropdownAnchor->IsOpen());
}

FReply SNamingTokensEditableTextBox::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	FReply Reply = FReply::Handled();
	if (InFocusEvent.GetCause() != EFocusCause::Cleared)
	{
		// Forward keyboard focus to our editable text widget
		Reply.SetUserFocus(EditableText.ToSharedRef(), InFocusEvent.GetCause());
	}

	return Reply;
}

void SNamingTokensEditableTextBox::OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath,
	const FFocusEvent& InFocusEvent)
{
	// We want to close the dropdown if whatever focus path doesn't contain our widgets.
	if (FilterWidget.IsValid() && !NewWidgetPath.ContainsWidget(this) && !NewWidgetPath.ContainsWidget(FilterWidget.Get()))
	{
		CloseSuggestionDropdown();
	}
}

FReply SNamingTokensEditableTextBox::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape && EditableText->HasKeyboardFocus())
	{
		// Clear focus
		return FReply::Handled().SetUserFocus(SharedThis(this), EFocusCause::Cleared);
	}
	// Other keydown events for navigation need to be handled in the SMultiLineEditableText callback OnEditableTextKeyDown,
	// because this callback will not always fire if bMultiline is true.
	
	return FReply::Unhandled();
}

void SNamingTokensEditableTextBox::EnterEditingMode()
{
	if (bSupportInlineEdit && !EditableText->IsTextReadOnly())
	{
		FSlateApplication::Get().SetKeyboardFocus(EditableText.ToSharedRef());
		EditableText->SelectAllText();

		SetVisibility(EVisibility::Visible);
		DisplayTokenIcon = true;
		DisplayBorderImage = true;
	}
}

void SNamingTokensEditableTextBox::ExitEditingMode()
{
	if (bSupportInlineEdit)
	{
		// Update the visibility to more closely match that of a normal read-only text block
		SetVisibility(EVisibility::HitTestInvisible);
		DisplayTokenIcon = false;
		DisplayBorderImage = false;
	}
}

bool SNamingTokensEditableTextBox::ValidateTokenizedText(const FText& InTokenizedText)
{
	FText ErrorMessage;

	if (!ValidateTokenArgs(InTokenizedText, ErrorMessage))
	{
		SetError(ErrorMessage);
		return false;
	}

	if (OnValidateTokenizedText.IsBound() && !OnValidateTokenizedText.Execute(InTokenizedText, ErrorMessage))
	{
		SetError(ErrorMessage);
		return false;
	}

	// Validate this last because unbalanced braces should not prevent the text from being committed
	if (!ValidateTokenBraces(InTokenizedText, ErrorMessage))
	{
		SetError(ErrorMessage);
		return true;
	}

	SetError(FText::GetEmpty());
	return true;
}

bool SNamingTokensEditableTextBox::ValidateTokenBraces(const FText& InTokenizedText, FText& OutError)
{
	// Get number of open and close braces in string
	int32 OpenBraceCount = 0;
	int32 CloseBraceCount = 0;

	bool bLastBraceWasOpen = false;

	auto GetUnbalancedBracesMessage = []()
	{
		return LOCTEXT("TemplateTextUnbalancedBracingError", "An unbalanced brace was detected. Please ensure that all braces are properly closed.");
	};

	FString TextString = InTokenizedText.ToString();
	for (const TCHAR& Char : TextString)
	{
		if (Char == TEXT('{'))
		{
			++OpenBraceCount;

			// didn't close the last brace
			if (bLastBraceWasOpen)
			{
				OutError = GetUnbalancedBracesMessage();
				break;
			}

			bLastBraceWasOpen = true;
		}
		else if (Char == TEXT('}'))
		{
			++CloseBraceCount;
			bLastBraceWasOpen = false;
		}
	}

	if (OpenBraceCount != CloseBraceCount)
	{
		OutError = GetUnbalancedBracesMessage();
	}

	return OutError.IsEmpty();
}

bool SNamingTokensEditableTextBox::ValidateTokenArgs(const FText& InTokenizedText, FText& OutError)
{
	TConstArrayView<FString> ValidArgs = ValidArguments.Get();
	if (ValidArgs.IsEmpty())
	{
		// Always valid if no provided valid args to check against
		return true;
	}

	TArray<FString> FoundArgs;
	if (!ParseArgs(InTokenizedText.ToString(), FoundArgs))
	{
		// No args in tokenized text
		return true;
	}

	// Not particularly performant
	for (const FString& FoundArg : FoundArgs)
	{
		if (!ValidArgs.ContainsByPredicate([FoundArg](const FString& InValidArg)
		{
			return InValidArg.Equals(FoundArg, ESearchCase::IgnoreCase);
		}))
		{
			OutError = LOCTEXT("TemplateTextInvalidArgError", "An argument/token was found that doesn't match any of the provided valid argument names.");
			return false;
		}
	}

	return true;
}

bool SNamingTokensEditableTextBox::ParseArgs(const FString& InTokenizedTextString, TArray<FString>& OutArgs)
{
	OutArgs = UE::NamingTokens::Utils::GetTokenKeysFromString(InTokenizedTextString);
	return !OutArgs.IsEmpty();
}

void SNamingTokensEditableTextBox::OnEditableTextChanged(const FText& InTokenizedText)
{
	OnTokenizedTextChanged.ExecuteIfBound(InTokenizedText);

	ValidateTokenizedText(InTokenizedText);

	UpdateFilters();
}

void SNamingTokensEditableTextBox::OnEditableTextCommitted(const FText& InTokenizedText, ETextCommit::Type InCommitType)
{
	// When the user right-clicks an unfocused textbox to open the editable-text context menu, the underlying
	// SMultiLineEditableText runs LoadText() from HandleMouseButtonDown before keyboard focus has actually transferred.
	// Our GetText() therefore still returns the resolved (evaluated) text and that gets latched as the editable buffer.
	// The eventual focus-loss commit then fires with the resolved text, which, if accepted, would overwrite the
	// tokenized template with its own evaluation result and break the periodic re-evaluation. 
	// 
	// This detects that case (InTokenizedText exactly equals the current ResolvedText) and ignores it.
	if (CanDisplayResolvedText.Get(true) && ResolvedText.IsSet() && !TokenizedText.Get().EqualTo(ResolvedText.Get()) && InTokenizedText.EqualTo(ResolvedText.Get()))
	{
		return;
	}

	if (DropdownAnchor.IsValid())
	{
		const EFocusCause DropdownFocusType = DropdownAnchor->HasAnyUserFocus().Get(EFocusCause::Cleared);

		if (InCommitType != ETextCommit::OnUserMovedFocus && DropdownFocusType == EFocusCause::Cleared)
		{
			CloseSuggestionDropdown();
		}
	}

	ExitEditingMode();

	if (!ValidateTokenizedText(InTokenizedText))
	{
		// The invalid text will not be committed, so clear the error message
		SetError(FText::GetEmpty());
		return;
	}

	if (!TokenizedText.IsBound())
	{
		TokenizedText.Set(InTokenizedText);
	}

	OnTokenizedTextCommitted.ExecuteIfBound(InTokenizedText, InCommitType);

	EvaluateNamingTokens();
}

void SNamingTokensEditableTextBox::UpdateFilters() const
{
	if (EditableText.IsValid() && FilterWidget.IsValid() && DropdownAnchor.IsValid() && DropdownAnchor->IsOpen())
	{
		// Find the text we're typing in between brackets and pass that as a filter to our suggestion drop down.

		if (const TSharedPtr<FCursorTokenData> TokenLocation = GetCurrentCursorTokenData())
		{
			FilterWidget->FilterTreeItems(TokenLocation->EnteredWord);
		}
	}
}

void SNamingTokensEditableTextBox::OpenOrCloseSuggestionDropdown() const
{
	if (DropdownAnchor.IsValid())
	{
		if (const TSharedPtr<FCursorTokenData> TokenLocation = GetCurrentCursorTokenData())
		{
			const bool bLastTokenDataChanged = !LastCursorTokenData.IsValid() || LastCursorTokenData->EnteredWord != TokenLocation->EnteredWord;
			LastCursorTokenData = TokenLocation;
			OpenSuggestionDropdown();
			if (bLastTokenDataChanged)
			{
				UpdateFilters();
			}
		}
		else
		{
			CloseSuggestionDropdown();
		}
	}
}

void SNamingTokensEditableTextBox::OpenOrCloseSuggestionDropdownNextTick() const
{
	if (!bModifyingSuggestionDropdown && DropdownAnchor.IsValid())
	{
		bModifyingSuggestionDropdown = true;
		
		const_cast<SNamingTokensEditableTextBox*>(this)->RegisterActiveTimer(0.f,
			FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
		{
			bModifyingSuggestionDropdown = false;
			OpenOrCloseSuggestionDropdown();
			return EActiveTimerReturnType::Stop;
		}));
	}
}

void SNamingTokensEditableTextBox::OpenSuggestionDropdown() const
{
	check(DropdownAnchor.IsValid());
	if (!DropdownAnchor->IsOpen())
	{
		DropdownAnchor->SetIsOpen(true, false);
	}
}

void SNamingTokensEditableTextBox::CloseSuggestionDropdown() const
{
	if (DropdownAnchor.IsValid() && DropdownAnchor->IsOpen())
	{
		DropdownAnchor->SetIsOpen(false, false);
	}
	LastCursorTokenData.Reset();
}

TSharedPtr<SNamingTokensEditableTextBox::FCursorTokenData> SNamingTokensEditableTextBox::GetCurrentCursorTokenData() const
{
	check(EditableText.IsValid());
	
	const FTextLocation Cursor = EditableText->GetCursorLocation();

	FString Line;
	EditableText->GetTextLine(Cursor.GetLineIndex(), Line);

	const int32 CursorOffset = Cursor.GetOffset();
	if (CursorOffset < 0 || CursorOffset >= Line.Len())
	{
		return nullptr;
	}

	// Look left for open '{'.
	const FString Left = Line.Left(CursorOffset);

	int32 LastOpen  = INDEX_NONE;
	int32 LastClose = INDEX_NONE;
	Left.FindLastChar(TEXT('{'), LastOpen);
	Left.FindLastChar(TEXT('}'), LastClose);

	// Error if no open bracket found, or a close bracket was found instead.
	if (LastOpen == INDEX_NONE || (LastClose != INDEX_NONE && LastClose > LastOpen))
	{
		return nullptr;
	}

	// Look right for close '}'.
	const FString Right   = Line.Mid(CursorOffset);

	const int32 NextClose = Right.Find(TEXT("}"), ESearchCase::CaseSensitive);
	const int32 NextOpen  = Right.Find(TEXT("{"), ESearchCase::CaseSensitive);

	// Error if no close bracket found, or an open bracket was found instead.
	if (NextClose == INDEX_NONE || (NextOpen != INDEX_NONE && NextOpen < NextClose))
	{
		return nullptr;
	}

	const int32 NextCloseOffset = CursorOffset + NextClose;

	TSharedRef<FCursorTokenData> CursorTokenData = MakeShared<FCursorTokenData>();
	CursorTokenData->StartLocation = FTextLocation(Cursor.GetLineIndex(), LastOpen + 1 /* Offset for token */);
	CursorTokenData->EndLocation   = FTextLocation(Cursor.GetLineIndex(), NextCloseOffset);
	CursorTokenData->Length        = NextCloseOffset - LastOpen;

	// Record word entered between brackets.
	{
		FString Between = Line.Mid(LastOpen + 1, NextCloseOffset - (LastOpen + 1));
		Between.TrimStartAndEndInline();

		// Alphanumeric, underscore, numbers, and also spaces, so we can account for display names.
		const FRegexPattern Pattern(TEXT("([A-Za-z0-9_ ]+(?::[A-Za-z0-9_ ]+)*:?)"));
		FRegexMatcher Matcher(Pattern, Between);
		
		FString CurrentWord;
		while (Matcher.FindNext())
		{
			CurrentWord = Matcher.GetCaptureGroup(1).TrimStartAndEnd();
			break;
		}

		CursorTokenData->EnteredWord = MoveTemp(CurrentWord);
	}
	
	return CursorTokenData;
}

TSharedRef<SWidget> SNamingTokensEditableTextBox::OnGetDropdownContent()
{
	if (!bEnableSuggestionDropdown)
	{
		return SNullWidget::NullWidget;
	}
	TSharedRef<SNamingTokenDataTreeViewWidget> Widget = SAssignNew(FilterWidget, SNamingTokenDataTreeViewWidget)
		.OnItemSelected(this, &SNamingTokensEditableTextBox::OnSuggestionCommitted)
		.OnFocused(this, &SNamingTokensEditableTextBox::OnSuggestionDropdownReceivedFocus)
		.FilterArgs(NamingTokenFilterArgs)
		.NamespacePriority(NamespaceSuggestionPriority);
	return Widget;
}

FReply SNamingTokensEditableTextBox::OnEditableTextKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent) const
{
	if (InCharacterEvent.GetCharacter() == TEXT('{'))
	{
		// Create a closing bracket.
		EditableText->DeleteSelectedText();
		const FTextLocation FilterCursorLocation = EditableText->GetCursorLocation();
		EditableText->InsertTextAtCursor(TEXT("}"));
		EditableText->GoTo(FilterCursorLocation);
	}
	else if (InCharacterEvent.GetCharacter() == TEXT('}'))
	{
		// If we are manually typing the closing bracket, and we already have one, simply move the cursor
		// to after the pre-existing closing bracket instead.
		const FTextLocation CursorLocation = EditableText->GetCursorLocation();
		
		FString Line;
		EditableText->GetTextLine(CursorLocation.GetLineIndex(), Line);
		// Make sure we're not already at the end of the text line.
		if (CursorLocation.GetOffset() < Line.Len())
		{
			const TCHAR NextCharacter = EditableText->GetCharacterAt(CursorLocation);
			if (NextCharacter == TEXT('}'))
			{
				const FTextLocation NextCursorLocation(CursorLocation.GetLineIndex(), CursorLocation.GetOffset() + 1);
				EditableText->GoTo(NextCursorLocation);
				return FReply::Handled();
			}
		}
	}
	
	return FReply::Unhandled();
}

FReply SNamingTokensEditableTextBox::OnEditableTextKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) const
{
	const FKey Key = InKeyEvent.GetKey();
	
	// We have to hook enter so we don't commit early. The user may be selecting a suggestion from the dropdown with enter.
	if (Key == EKeys::Enter && DropdownAnchor.IsValid() && DropdownAnchor->IsOpen() && FilterWidget.IsValid())
	{
		OnSuggestionSelected(FilterWidget->GetSelectedItem());
		return FReply::Handled();
	}

	if (!(Key == EKeys::Escape || Key == EKeys::Tab || Key == EKeys::Enter))
	{
		// If we're not submitting text then we want to always check if the suggestion box should open or close.
		// Use a timer to ensure our updated text is submitted before we check cursor location.
		OpenOrCloseSuggestionDropdownNextTick();
	}
	
	if (DropdownAnchor.IsValid() && DropdownAnchor->IsOpen() && FilterWidget.IsValid()
		&& FilterWidget->ForwardKeyEventForNavigation(InKeyEvent))
	{
		// Set focus back to the editable text so the user can keep typing.
		return FReply::Handled().SetUserFocus(EditableText.ToSharedRef(), EFocusCause::OtherWidgetLostFocus);
	}
	
	return FReply::Unhandled();
}

void SNamingTokensEditableTextBox::OnEditableTextCursorMoved(const FTextLocation& NewCursorPosition)
{
	const FTextSelection CurrentSelection = EditableText->GetSelection();
    const FTextLocation& Begin = CurrentSelection.GetBeginning();
    const FTextLocation& End = CurrentSelection.GetEnd();
	
	// FTextLocation has limited operator support.
	const bool bIsSelected = (Begin.IsValid() && End.IsValid() && Begin < End) && (((Begin < NewCursorPosition) && (NewCursorPosition < End))
		|| (NewCursorPosition == Begin || NewCursorPosition == End));

	// If we're selecting text, avoid opening the suggestion window.
	if (!bIsSelected || (DropdownAnchor.IsValid() && DropdownAnchor->IsOpen()))
	{
		OpenOrCloseSuggestionDropdown();
	}
}

void SNamingTokensEditableTextBox::OnSuggestionSelected(TSharedPtr<FNamingTokenDataTreeItem> SelectedItem) const
{
	if (SelectedItem.IsValid())
	{
		if (const TSharedPtr<FCursorTokenData> TokenLocation = GetCurrentCursorTokenData())
		{
			// Either insert just the token, or the namespace:token.
			bool bIsFullyQualified = !SelectedItem->bIsGlobal || bFullyQualifyGlobalTokenSuggestions;
			if (!bFullyQualifyFilteredNamespaces && NamingTokenFilterArgs.AdditionalNamespacesToInclude.Contains(SelectedItem->Namespace))
			{
				bIsFullyQualified = false;
			}
		
			FString TextToInsert = SelectedItem->ToString(bIsFullyQualified);
			if (SelectedItem->IsNamespace())
			{
				// Add the delimiter and continue the filter process.
				TextToInsert += UE::NamingTokens::Utils::GetNamespaceDelimiter();
			}
			else
			{
				// We're finished if this is a token being inserted.
				CloseSuggestionDropdown();
			}
			
			// Set our cursor to the start and select everything in our range.
			EditableText->GoTo(TokenLocation->StartLocation);
			EditableText->SelectText(TokenLocation->StartLocation, TokenLocation->EndLocation);

			// Insert all of our text.
			EditableText->InsertTextAtCursor(FText::FromString(TextToInsert));
			
			if (SelectedItem->IsNamespace())
			{
				UpdateFilters();
			}
			else if (const TSharedPtr<FCursorTokenData> FinalizedTokenLocation = GetCurrentCursorTokenData())
            {
                // If we're inserting the final token offset the cursor to after the final bracket.
                const FTextLocation AfterBracket(FinalizedTokenLocation->EndLocation.GetLineIndex(), FinalizedTokenLocation->EndLocation.GetOffset() + 1);
                EditableText->GoTo(AfterBracket);
            }
		}
	}
}

void SNamingTokensEditableTextBox::OnSuggestionCommitted(TSharedPtr<FNamingTokenDataTreeItem> SelectedItem)
{
	// Insert the selected item to our text box.
	OnSuggestionSelected(SelectedItem);
	// Gather the plain text from the text box lines, because GetText() returns OUR binding, which hasn't been updated yet.
	const FText EnteredText = EditableText->GetPlainText();
	// Committing from the suggestion box means the suggestion box had focus, and OUR text commit event has already fired. We need to
	// run our commit logic to update bindings and validate the token.
	OnEditableTextCommitted(EnteredText, ETextCommit::Type::OnUserMovedFocus);

	// Focus the text box so the user can keep typing. Selecting a suggestion doesn't mean the user wanted text input to stop.
	FSlateApplication::Get().SetKeyboardFocus(EditableText, EFocusCause::OtherWidgetLostFocus);
	FSlateApplication::Get().SetUserFocus(0, EditableText, EFocusCause::OtherWidgetLostFocus);
}

void SNamingTokensEditableTextBox::OnSuggestionDropdownReceivedFocus() const
{
	// Without this focus event, the keyboard focus will remain on the dropdown and prevent the user from typing,
	// moving the cursor, or pressing tab or enter to confirm a selection. Keyboard navigation of the dropdown
	// is handled separately and requires our editable text box to maintain focus.
	FSlateApplication::Get().SetKeyboardFocus(EditableText, EFocusCause::WindowActivate);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE