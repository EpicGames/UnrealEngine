// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MultiLineEditableText.h"
#include "NamingTokensEngineSubsystem.h"

#include "NamingTokensEditableText.generated.h"

class SNamingTokensEditableTextBox;
struct FEditableTextBoxStyle;

/**
 * Editable text for displaying tokenized strings in either their unevaluated or resolved form.
 */
UCLASS(MinimalAPI, Config = NamingTokens, DefaultConfig, ClassGroup = UI, PrioritizeCategories = ("NamingTokens", "Content", "Appearance", "Behavior"), meta = (Category = "NamingTokens UI", DisplayName = "Naming Tokens Editable Text Box"))
class UNamingTokensEditableText : public UMultiLineEditableText
{
	GENERATED_BODY()

public:
	DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnPreEvaluateNamingTokens);
	
	NAMINGTOKENSUI_API UNamingTokensEditableText();
	
	/** Retrieve the evaluated text. */
	UFUNCTION(BlueprintCallable, Category=NamingTokens)
	NAMINGTOKENSUI_API const FText& GetResolvedText() const;

	/** Retrieve the raw token text. */
	UFUNCTION(BlueprintCallable, Category=NamingTokens)
	NAMINGTOKENSUI_API const FText& GetTokenizedText() const;

	/** Sets the argument style to use for the widget. */
	UFUNCTION(BlueprintSetter)
	NAMINGTOKENSUI_API void SetWidgetArgumentStyle(const FTextBlockStyle& InWidgetStyle);

	/** Set the background color. */
	UFUNCTION(BlueprintSetter)
	NAMINGTOKENSUI_API void SetBackgroundColor(const FSlateColor& InColor);
	
	/** Set the contexts to use during evaluation. */
	UFUNCTION(BlueprintSetter)
	NAMINGTOKENSUI_API void SetContexts(const TArray<UObject*>& InContexts);

	/** Set whether the token icon is displayed. */
	UFUNCTION(BlueprintSetter)
	NAMINGTOKENSUI_API void SetDisplayTokenIcon(bool bValue);

	/** Set whether error messages are displayed for token formatting. */
	UFUNCTION(BlueprintSetter)
	NAMINGTOKENSUI_API void SetDisplayErrorMessage(bool bValue);

	/** Set the background color of the border image. */
	UFUNCTION(BlueprintSetter)
	NAMINGTOKENSUI_API void SetDisplayBorderImage(bool bValue);

	/** If the token icon should be displayed. */
	UFUNCTION(BlueprintGetter)
	NAMINGTOKENSUI_API bool GetDisplayTokenIcon() const;

	/** If the error icon should be displayed. */
	UFUNCTION(BlueprintGetter)
	NAMINGTOKENSUI_API bool GetDisplayErrorMessage() const;

	/** If the border image should be displayed. */
	UFUNCTION(BlueprintGetter)
	NAMINGTOKENSUI_API bool GetDisplayBorderImage() const;

	/** Set whether we can display resolved text or not. */
	UFUNCTION(BlueprintSetter)
	NAMINGTOKENSUI_API void SetCanDisplayResolvedText(bool bValue);

	/** Get if we can display resolved text. */
	UFUNCTION(BlueprintGetter)
	NAMINGTOKENSUI_API bool GetCanDisplayResolvedText() const;

	/** Set whether the widget should warn when token(s) in the tokenized text are not recognized. */
	UFUNCTION(BlueprintSetter)
	NAMINGTOKENSUI_API void SetShowUnknownTokenWarning(bool bValue);

	/** Get whether the widget warns when token(s) in the tokenized text are not recognized. */
	UFUNCTION(BlueprintGetter)
	NAMINGTOKENSUI_API bool GetShowUnknownTokenWarning() const;

	/** Set whether the widget should warn when token(s) in the tokenized text evaluate to an empty value. */
	UFUNCTION(BlueprintSetter)
	NAMINGTOKENSUI_API void SetShowUnsetTokenWarning(bool bValue);

	/** Get whether the widget warns when token(s) in the tokenized text evaluate to an empty value. */
	UFUNCTION(BlueprintGetter)
	NAMINGTOKENSUI_API bool GetShowUnsetTokenWarning() const;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
protected:
	//~ Begin UWidget Interface
	virtual TSharedRef<SWidget> RebuildWidget() override;
	virtual void SynchronizeProperties() override;
	// End of UWidget

	//~ Begin UVisual Interface
	virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End UVisual Interface
	
private:
	/** Calculate if we're allowed to display the resolved text. */
	bool CanDisplayResolvedTextAttribute() const;

	/** Callback when we are evaluating tokens. */
	void OnPreEvaluateNamingTokensCallback();
	
protected:
	/** Set naming token filter args to use during token evaluation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=NamingTokens)
	FNamingTokenFilterArgs FilterArgs;

	/** A list prioritizing namespaces to show first in the dropdown. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Behavior, meta = (EditCondition="bEnableSuggestionDropdown", EditConditionHides, DisplayAfter=bFullyQualifyFilteredNamespaces))
	TArray<FString> NamespaceSuggestionPriority;
	
	/** If the dropdown suggestion box should be enabled. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Behavior)
	bool bEnableSuggestionDropdown = true;

	/** If namespaces included in the FilterArgs.AdditionalNamespacesToInclude should autocomplete. If false, only the token is inserted. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Behavior, meta = (EditCondition="bEnableSuggestionDropdown", EditConditionHides))
	bool bFullyQualifyFilteredNamespaces = false;
	
	/** If this text box is configured for multiline. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	bool bIsMultiline = false;

	/** Display the token icon in the text box. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, BlueprintSetter=SetDisplayTokenIcon, BlueprintGetter=GetDisplayTokenIcon, Category=Appearance)
	bool bDisplayTokenIcon = false;

	/** Display an error message when tokens aren't properly formatted. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, BlueprintSetter=SetDisplayErrorMessage, BlueprintGetter=GetDisplayErrorMessage, Category=Appearance)
	bool bDisplayErrorMessage = false;

	/** Display the border image. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, BlueprintSetter=SetDisplayBorderImage, BlueprintGetter=GetDisplayBorderImage, Category=Appearance)
	bool bDisplayBorderImage = false;

	/** The font to apply to the non-resolved tokens. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, BlueprintSetter=SetWidgetArgumentStyle, Category=Appearance)
	FTextBlockStyle ArgumentStyle;
	
	/** Set the background color of the border image. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, BlueprintSetter=SetBackgroundColor, Category=Appearance)
	FSlateColor BackgroundColor;

	/** If the resolved text can be displayed. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, BlueprintSetter=SetCanDisplayResolvedText, BlueprintGetter=GetCanDisplayResolvedText, Category=Appearance)
	bool bCanDisplayResolvedText = true;

	/** Display a warning when token(s) in the tokenized text are not recognized (unknown tokens). */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, BlueprintSetter=SetShowUnknownTokenWarning, BlueprintGetter=GetShowUnknownTokenWarning, Category=Appearance)
	bool bShowUnknownTokenWarning = true;

	/** Display a warning when token(s) in the tokenized text evaluate to an empty value (unset tokens). */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadWrite, BlueprintSetter=SetShowUnsetTokenWarning, BlueprintGetter=GetShowUnsetTokenWarning, Category=Appearance)
	bool bShowUnsetTokenWarning = false;
	
#if WITH_EDITORONLY_DATA
	/** If the resolved text should display while designing in the UMG window. If false, only the tokenized text will be displayed in the UMG designer. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Appearance, meta = (EditCondition="bCanDisplayResolvedText", EditConditionHides))
	bool bDisplayResolvedTextInDesigner = true;
#endif
	
public:
	/** Called before naming token evaluation. */
	UPROPERTY(BlueprintAssignable, Category="NamingTokens|Event")
	FOnPreEvaluateNamingTokens OnPreEvaluateNamingTokens;
	
private:
	/** The editable text slate widget. */
	TSharedPtr<SNamingTokensEditableTextBox> MyNamingTokensEditableText;

	/** The editable text box style we use. */
	TSharedPtr<FEditableTextBoxStyle> EditableTextBoxStyle;

	/** Contexts to include during our evaluation. */
	UPROPERTY(BlueprintReadWrite, BlueprintSetter=SetContexts, Category=NamingTokens, meta=(AllowPrivateAccess))
	TArray<TObjectPtr<UObject>> Contexts;
};
