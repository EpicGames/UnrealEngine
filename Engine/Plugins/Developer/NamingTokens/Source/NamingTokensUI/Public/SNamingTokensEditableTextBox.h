// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Misc/Attribute.h"
#include "NamingTokensEngineSubsystem.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"

class ITextLayoutMarshaller;
class FNamingTokensStringSyntaxHighlighterMarshaller;
class FString;
class FText;
class SMenuAnchor;

struct FNamingTokenDataTreeItem;

DECLARE_DELEGATE_OneParam(FOnTokenizedTextEvaluated, const FText&);

/**
 * An editable text box for displaying tokenized strings in either their unevaluated or resolved form.
 */
class SNamingTokensEditableTextBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SNamingTokensEditableTextBox)
		: _Style(nullptr)
		, _ArgumentStyle(nullptr)
		, _ShouldEvaluateTokens(true)
		, _EvaluationFrequency(0.0f)
		, _AllowMultiLine(false)
		, _IsReadOnly(false)
		, _CanDisplayResolvedText(true)
		, _DisplayTokenIcon(true)
		, _DisplayErrorMessage(true)
		, _DisplayBorderImage(true)
		, _TextBoxPadding()
		, _SupportInlineEdit(false)
		, _EnableSuggestionDropdown(true)
		, _FullyQualifyGlobalTokenSuggestions(false)
		, _FullyQualifyFilteredNamespaces(false)
		, _ShowUnknownTokenWarning(true)
		, _ShowUnsetTokenWarning(false)
	{}
		/** The styling of the textbox. */
		SLATE_STYLE_ARGUMENT(FEditableTextBoxStyle, Style)

		/** The styling of our arguments. */
		SLATE_STYLE_ARGUMENT(FTextBlockStyle, ArgumentStyle)
		
		/** The initial text that will appear in the widget. */
		SLATE_ATTRIBUTE(FText, Text)

		/** The (optional) resolved text, displayed when the text box is not focused. If not provided, the templated text is shown. */
		SLATE_ATTRIBUTE(FText, ResolvedText)

		/** The list of available arguments to use in this template string. If Empty, any argument name is valid. */
		SLATE_ATTRIBUTE(TConstArrayView<FString>, ValidArguments)

		/** The marshaller used to get/set the raw text to/from the text layout. */
		SLATE_ARGUMENT(TSharedPtr<ITextLayoutMarshaller>, Marshaller)

		/** Whether this widget should evaluate the tokenized string */
		SLATE_ATTRIBUTE(bool, ShouldEvaluateTokens)

		/** If set, specifies how frequently the tokenized text should be re-evaluated */
		SLATE_ATTRIBUTE(float, EvaluationFrequency)

		/** Whether to allow multi-line text. */
		SLATE_ATTRIBUTE(bool, AllowMultiLine)

		/** Sets whether this text box can actually be modified interactively by the user */
		SLATE_ATTRIBUTE(bool, IsReadOnly)

		/** If we're allowed to display the resolved text. */
		SLATE_ATTRIBUTE(bool, CanDisplayResolvedText)

		/** If we can display the default token icon. */
		SLATE_ATTRIBUTE(bool, DisplayTokenIcon)

		/** If we display error messages. */
		SLATE_ATTRIBUTE(bool, DisplayErrorMessage)

		/** If we should use the border image. */
		SLATE_ATTRIBUTE(bool, DisplayBorderImage)

		/** Padding for the editable textbox (overrides the textbox style)*/
		SLATE_ATTRIBUTE(FMargin, TextBoxPadding)

		/** Allows custom validation before committing the tokenized text. */
		SLATE_EVENT(FOnVerifyTextChanged, OnValidateTokenizedText)

		/** Allows custom validation after the text has been resolved. */
		SLATE_EVENT(FOnVerifyTextChanged, OnValidateResolvedText)

		/** Called whenever the (tokenized) text is changed interactively by the user. */
		SLATE_EVENT(FOnTextChanged, OnTextChanged)
		
		/** Called whenever the (tokenized) text is committed by the user. */
		SLATE_EVENT(FOnTextCommitted, OnTextCommitted)

		/** Called prior to Naming Token evaluation. Only fires if this widget is handling evaluations. */
		SLATE_EVENT(FSimpleDelegate, OnPreEvaluateNamingTokens)

		/** Called after the tokenized text is evaluated to provide the resulting resolved text. */
		SLATE_EVENT(FOnTokenizedTextEvaluated, OnTokenizedTextEvaluated)

		/** Filter args to apply during evaluation. */
		SLATE_ARGUMENT(FNamingTokenFilterArgs, FilterArgs)

		/** Contexts to provide to evaluation. */
		SLATE_ARGUMENT(TArray<TObjectPtr<UObject>>, Contexts)

		/** 
		 * If true, the look and feel of the editable textbox will be treated like an inline editable textbox.
		 * This overrides the DisplayTokenIcon and DisplayBorderImage attributes, as well as the visibility
		 */
		SLATE_ARGUMENT(bool, SupportInlineEdit)

		/** If drop down suggestions should be enabled. */
		SLATE_ARGUMENT(bool, EnableSuggestionDropdown)
		
		/** If the namespace should be inserted when selecting a global suggestion. */
		SLATE_ARGUMENT(bool, FullyQualifyGlobalTokenSuggestions)

		/** If namespaces included in FilterArgs.AdditionalNamespacesToInclude should be autocompleted. */
		SLATE_ARGUMENT(bool, FullyQualifyFilteredNamespaces)

		/** Set the namespace priority to use when sorting dropdown suggestions. */
		SLATE_ARGUMENT(TArray<FString>, NamespaceSuggestionPriority)

		/** If the widget should display a warning when token(s) in the tokenized text are not recognized (unknown tokens). */
		SLATE_ARGUMENT(bool, ShowUnknownTokenWarning)

		/** If the widget should display a warning when token(s) in the tokenized text evaluate to an empty value (unset tokens). */
		SLATE_ARGUMENT(bool, ShowUnsetTokenWarning)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	NAMINGTOKENSUI_API void Construct(const FArguments& InArgs);

	/** Retrieve the editable text widget. */
	TSharedPtr<SMultiLineEditableText> GetEditableText() const { return EditableText; }

	/** Set the current tokenized text. */
	NAMINGTOKENSUI_API void SetTokenizedText(const FText& InText);
	
	/** Return the bound text attribute. */
	const TAttribute<FText>& GetBoundText() const { return BoundText; }

	/** Retrieve the resolved text. */
	NAMINGTOKENSUI_API const FText& GetResolvedText() const;

	/** Retrieve the raw tokenized text. */
	NAMINGTOKENSUI_API const FText& GetTokenizedText() const;

	/** Sets the style to use. */
	NAMINGTOKENSUI_API void SetNormalStyle(const FEditableTextBoxStyle* InNormalStyle);

	/** Sets the argument style to use. */
	NAMINGTOKENSUI_API void SetArgumentStyle(const FTextBlockStyle* InArgumentStyle);

	/** If the textbox should look and behave like an inline editable textbox */
	NAMINGTOKENSUI_API void SetSupportInlineEdit(bool bInSupportInlineEdit);

	/** If the suggestion box should be enabled. */
	NAMINGTOKENSUI_API void SetEnableSuggestionDropdown(bool bInEnableSuggestionDropdown);
	
	/** Provide filter args to use during evaluation. */
	NAMINGTOKENSUI_API void SetFilterArgs(const FNamingTokenFilterArgs& InFilterArgs);

	/** Set whether namespaces under FilterArgs.AdditionalNamespacesToInclude should autocomplete. */
	NAMINGTOKENSUI_API void SetFullyQualifyFilteredNamespaces(bool bNewValue);

	/** Set the namespace priority to use when sorting dropdown suggestions. */
	NAMINGTOKENSUI_API void SetNamespaceSuggestionPriority(const TArray<FString>& InNamespaces);

	/** Provide contexts to naming token evaluation. */
	NAMINGTOKENSUI_API void SetContexts(const TArray<UObject*>& InContexts);

	/** Set whether the widget should warn when token(s) in the tokenized text are not recognized. */
	NAMINGTOKENSUI_API void SetShowUnknownTokenWarning(bool bNewValue);

	/** Set whether the widget should warn when token(s) in the tokenized text evaluate to an empty value. */
	NAMINGTOKENSUI_API void SetShowUnsetTokenWarning(bool bNewValue);

	/** Evaluate the token text and update the resolved text. */
	NAMINGTOKENSUI_API void EvaluateNamingTokens();
	
	/** If inline editing is supported, gives focus to the editable text, and sets the visibility and display options to resemble an editable textbox */
	NAMINGTOKENSUI_API void EnterEditingMode();

	/** If inline editing is supported, sets the visibility and display options to resemble a read-only textblock */
	NAMINGTOKENSUI_API void ExitEditingMode();

private:
	/** @return Text contents based on current state */
	FText GetText() const;

	const FSlateBrush* GetBorderImage() const;
	/** The background color for the border. */
	NAMINGTOKENSUI_API virtual FSlateColor GetBorderBackgroundColor() const;
	NAMINGTOKENSUI_API virtual FSlateColor GetForegroundColor() const override;

	void SetError(const FText& InError) const;

	NAMINGTOKENSUI_API virtual bool SupportsKeyboardFocus() const override;
	NAMINGTOKENSUI_API virtual bool HasKeyboardFocus() const override;
	NAMINGTOKENSUI_API virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	NAMINGTOKENSUI_API virtual void OnFocusChanging(const FWeakWidgetPath& PreviousFocusPath, const FWidgetPath& NewWidgetPath, const FFocusEvent& InFocusEvent) override;
	NAMINGTOKENSUI_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	bool ValidateTokenizedText(const FText& InTokenizedText);
	bool ValidateTokenBraces(const FText& InTokenizedText, FText& OutError);
	bool ValidateTokenArgs(const FText& InTokenizedText, FText& OutError);

	bool ParseArgs(const FString& InTokenizedTextString, TArray<FString>& OutArgs);

	void OnEditableTextChanged(const FText& InTokenizedText);
	void OnEditableTextCommitted(const FText& InTokenizedText, ETextCommit::Type InCommitType);

	/** Check for type text and update filters if needed. */
	void UpdateFilters() const;

	struct FCursorTokenData
	{
		/** Start '{' location of token. */
		FTextLocation StartLocation;
		/** End '}' location of token.  */
		FTextLocation EndLocation;
		/** Length of EndLocation-StartLocation offset. */
		int32 Length;
		/** Word entered between brackets. */
		FString EnteredWord;
	};

	/** Opens or closes the dropdown based on cursor status. */
	void OpenOrCloseSuggestionDropdown() const;
	/** Opens or closes the suggestion drop down on the next tick, providing a request hasn't already been made. */
	void OpenOrCloseSuggestionDropdownNextTick() const;
	
	/** Trigger the suggestion process. */
	void OpenSuggestionDropdown() const;
	/** Close the suggestion window. */
	void CloseSuggestionDropdown() const;

	/** Retrieve information about the current token where the cursor is. */
	TSharedPtr<FCursorTokenData> GetCurrentCursorTokenData() const;
	
	/** Create the dropdown content for filtering naming tokens. */
	TSharedRef<SWidget> OnGetDropdownContent();
	
	/** When our editable text has a character typed. */
	FReply OnEditableTextKeyChar(const FGeometry& MyGeometry, const FCharacterEvent& InCharacterEvent) const;

	/** When editable text has a key pressed. */
	FReply OnEditableTextKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) const;

	/** User has moved the text cursor. */
	void OnEditableTextCursorMoved(const FTextLocation& NewCursorPosition);
	
	/** When a dropdown suggestion has been selected. */
	void OnSuggestionSelected(TSharedPtr<FNamingTokenDataTreeItem> SelectedItem) const;

	/** Called when the user commits a suggestion (single click, double click, Enter, or Tab). */
	void OnSuggestionCommitted(TSharedPtr<FNamingTokenDataTreeItem> SelectedItem);

	/** Called if the suggestion box has received focus. */
	void OnSuggestionDropdownReceivedFocus() const;
	
	/** Builds the tooltip shown on the unresolved token warning icon */
	FText GetUnresolvedTokenWarningTooltip() const;

private:
	/** Filter args to use during evaluation. */
	FNamingTokenFilterArgs NamingTokenFilterArgs;

	/** Contexts to provide to evaluation. */
	TArray<TObjectPtr<UObject>> NamingTokenContexts;

	/** A list prioritizing namespaces to show first in the dropdown. */
	TArray<FString> NamespaceSuggestionPriority;
	
	/** Editable text widget. */
	TSharedPtr<SMultiLineEditableText> EditableText;

	/** The dropdown filter UI anchor. */
	TSharedPtr<SMenuAnchor> DropdownAnchor;

	/** The Dropdown widget. */
	TSharedPtr<class SNamingTokenDataTreeViewWidget> FilterWidget;

	/** Cursor token data of the last token data. */
	mutable TSharedPtr<FCursorTokenData> LastCursorTokenData;
	
	/** Style shared between Editable and NonEditable text widgets. */
	const FEditableTextBoxStyle* TextBoxStyle = nullptr;
	/** The style to use for our argument formatting. */
	const FTextBlockStyle* ArgumentStyle = nullptr;

	TSharedPtr<SHorizontalBox> Box;

	TSharedPtr<SBox> HScrollBarBox;
	TSharedPtr<SScrollBar> HScrollBar;

	TSharedPtr<SBox> VScrollBarBox;
	TSharedPtr<SScrollBar> VScrollBar;

	/** The marshaller we use. */
	TSharedPtr<FNamingTokensStringSyntaxHighlighterMarshaller> Marshaller;
	
	TSharedPtr<class IErrorReportingWidget> ErrorReporting;

	/** The bound text which could return either tokenized or resolved text. */
	TAttribute<FText> BoundText;
	
	TAttribute<FText> TokenizedText;
	TAttribute<FText> ResolvedText;

	/** If true, this widget will evaluate the tokenized text to produce the resolved text */
	TAttribute<bool> ShouldEvaluateTokens;

	/** If set, specifies how frequently the tokenized text should be re-evaluated */
	TAttribute<float> EvaluationFrequency;

	/** If we are displaying the token icon. */
	TAttribute<bool> DisplayTokenIcon;

	/** If error messages are displayed. */
	TAttribute<bool> DisplayErrorMessage;
	
	/** If the border image is used. */
	TAttribute<bool> DisplayBorderImage;
	
	/** Textbox padding (overrides style) */
	TAttribute<FMargin> TextBoxPaddingOverride;

	/** Attribute checking if we can display the resolved text. */
	TAttribute<bool> CanDisplayResolvedText;
	TAttribute<TConstArrayView<FString>> ValidArguments;
	
	/** Callback to verify tokenized text when changed. Will return an error message to denote problems. */
    FOnVerifyTextChanged OnValidateTokenizedText;

	/** Callback to verify resolved text when changed. Will return an error message to denote problems. */
	FOnVerifyTextChanged OnValidateResolvedText;

	/** Callback when tokenized text is changed. */
	FOnTextChanged OnTokenizedTextChanged;

	/** Callback when tokenized text is committed. */
	FOnTextCommitted OnTokenizedTextCommitted;

	/** Callback before we evaluate tokens. */
	FSimpleDelegate OnPreEvaluateTokens;

	/** Callback after the tokenized text has been evaluated */
	FOnTokenizedTextEvaluated OnTokenizedTextEvaluated;

	/** If global tokens should have their namespace inserted too from suggestions. */
	bool bFullyQualifyGlobalTokenSuggestions = false;

	/** If namespaces under FilterArgs.AdditionalNamespacesToInclude should autocomplete. */
	bool bFullyQualifyFilteredNamespaces = false;
	
	/** If the widget supports inline editing */
	bool bSupportInlineEdit = false;

	/** If suggestions are enabled by default. */
	bool bEnableSuggestionDropdown = true;

	/** Suggesting box in the process of opening or closing. */
	mutable bool bModifyingSuggestionDropdown = false;

	/** If the widget should warn when token(s) in the tokenized text are not recognized (unknown tokens). */
	bool bShowUnknownTokenWarning = true;

	/** If the widget should warn when token(s) in the tokenized text evaluate to an empty value (unset tokens). */
	bool bShowUnsetTokenWarning = false;

	/** Token keys that were unknown during the last evaluation. Drives the warning icon visibility and tooltip. */
	TArray<FString> UnknownTokenKeys;

	/** Token keys that evaluated to empty values during the last evaluation. Drives the warning icon visibility and tooltip. */
	TArray<FString> UnsetTokenKeys;
};
