// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class NamingTokensUncookedOnly : ModuleRules
{
    public NamingTokensUncookedOnly(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "BlueprintGraph",
	            "Blutility",
	            "Core",
                "CoreUObject",
                "Engine",
                "NamingTokens",
                "UnrealEd",
            }
        );
    }
}