// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokensFactory.h"

#include "NamingTokens.h"

#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "K2Node_Event.h"
#include "Kismet2/KismetEditorUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NamingTokensFactory)

UNamingTokensFactory::UNamingTokensFactory()
{
	SupportedClass = UNamingTokens::StaticClass();

	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UNamingTokensFactory::FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context,
	FFeedbackContext* Warn)
{
	UClass* ParentClass = SupportedClass;
	check(InClass->IsChildOf(ParentClass));
    
	UBlueprint* NewBP = FKismetEditorUtilities::CreateBlueprint(
		ParentClass,
		InParent,
		InName,
		BPTYPE_Normal,
		UBlueprint::StaticClass(),
		UBlueprintGeneratedClass::StaticClass()
	);

	return NewBP;
}
