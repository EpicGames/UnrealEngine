// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"

#include "NamingTokensFactory.generated.h"

UCLASS(MinimalAPI)
class UNamingTokensFactory : public UFactory
{
	GENERATED_BODY()

public:
	UNamingTokensFactory();
	
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
};