// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamingTokensUncookedOnlyModule.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "EditorUtilityBlueprint.h"
#include "NamingTokens.h"
#include "Utils/NamingTokensUncookedOnlyUtils.h"

void FNamingTokensUncookedOnlyModule::StartupModule()
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	AssetAddedHandle = AssetRegistry.OnAssetAdded().AddRaw(this, &FNamingTokensUncookedOnlyModule::OnAssetAdded);
}

void FNamingTokensUncookedOnlyModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded("AssetRegistry"))
	{
		IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
		AssetRegistry.OnAssetAdded().Remove(AssetAddedHandle);
	}
}

void FNamingTokensUncookedOnlyModule::OnAssetAdded(const FAssetData& InAssetData)
{
	// Look for any Naming Token blueprints being created and make sure we have the correct starting nodes.
	// We can't rely on a single factory because we have to account for Editor Utility Blueprint versions too.
	
	// Check with fast methods to determine if we are dealing with our asset or not. The asset should
	// already be loaded into memory.
	if ((InAssetData.IsValid() && !InAssetData.IsRedirector() && !InAssetData.HasAnyPackageFlags(PKG_Cooked) && InAssetData.IsAssetLoaded())
		&& (InAssetData.AssetClassPath == UBlueprint::StaticClass()->GetClassPathName()
			|| InAssetData.AssetClassPath == UEditorUtilityBlueprint::StaticClass()->GetClassPathName()))
	{
		FString NativeParentPathStr;
		if (!InAssetData.GetTagValue(FBlueprintTags::NativeParentClassPath, NativeParentPathStr))
		{
			return;
		}
		const FSoftClassPath NativeParentPath(NativeParentPathStr);
		const UClass* NativeParentClass = NativeParentPath.ResolveClass();
		if (!NativeParentClass)
		{
			NativeParentClass = NativeParentPath.TryLoadClass<UObject>();
		}
			
		if (NativeParentClass && NativeParentClass->IsChildOf(UNamingTokens::StaticClass()))
		{
			UBlueprint* Blueprint = CastChecked<UBlueprint>(InAssetData.GetAsset());
			if (Blueprint->bIsNewlyCreated)
			{
				UE::NamingTokens::Utils::UncookedOnly::Private::SetupInitialBlueprint(Blueprint);
			}
		}
	}
}

IMPLEMENT_MODULE(FNamingTokensUncookedOnlyModule, NamingTokensUncookedOnly)
