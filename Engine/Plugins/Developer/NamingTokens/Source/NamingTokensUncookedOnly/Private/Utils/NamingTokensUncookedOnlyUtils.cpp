// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/NamingTokensUncookedOnlyUtils.h"

#include "BlueprintEditorSettings.h"
#include "EdGraphSchema_K2.h"
#include "K2Node_Event.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "NamingTokens.h"

void UE::NamingTokens::Utils::UncookedOnly::Private::SetupInitialBlueprint(UBlueprint* InBlueprint)
{
	const UBlueprintEditorSettings* Settings = GetDefault<UBlueprintEditorSettings>();
	if (Settings && Settings->bSpawnDefaultBlueprintNodes)
	{
		// Create default events.
		int32 NodePositionY = 0;

		UEdGraph* EventGraph = FindObject<UEdGraph>(InBlueprint, *(UEdGraphSchema_K2::GN_EventGraph.ToString()));
		check(EventGraph);
		
		if (UK2Node_Event* OnPreEvaluateNode = FKismetEditorUtilities::AddDefaultEventNode(InBlueprint, EventGraph,
			UNamingTokens::GetOnPreEvaluateFunctionName(), UNamingTokens::StaticClass(), NodePositionY))
		{
			// Set the node comment. The comment is displayed because we are automatically placed as ghost nodes.
			// Once a connection is made the comment will go away, but is still accessible via tooltip.
			const UFunction* Function = UNamingTokens::StaticClass()->FindFunctionByName(UNamingTokens::GetOnPreEvaluateFunctionName());
			check(Function);
			OnPreEvaluateNode->NodeComment = Function->GetToolTipText().ToString();
		}
		if (UK2Node_Event* OnPostEvaluateNode = FKismetEditorUtilities::AddDefaultEventNode(InBlueprint, EventGraph,
			UNamingTokens::GetOnPostEvaluateFunctionName(), UNamingTokens::StaticClass(), NodePositionY))
		{
			// Set the node comment. The comment is displayed because we are automatically placed as ghost nodes.
			// Once a connection is made the comment will go away, but is still accessible via tooltip.
			const UFunction* Function = UNamingTokens::StaticClass()->FindFunctionByName(UNamingTokens::GetOnPostEvaluateFunctionName());
			check(Function);
			OnPostEvaluateNode->NodeComment = Function->GetToolTipText().ToString();
		}
	}
}
