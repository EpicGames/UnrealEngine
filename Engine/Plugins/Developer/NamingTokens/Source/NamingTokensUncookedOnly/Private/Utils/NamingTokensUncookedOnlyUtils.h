// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UBlueprint;

namespace UE::NamingTokens::Utils::UncookedOnly::Private
{
	/** Configure a new blueprint for Naming Token use. */
	void SetupInitialBlueprint(UBlueprint* InBlueprint);
}
